// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class LevelSequence : ModuleRules
{
	public LevelSequence(ReadOnlyTargetRules Target) : base(Target)
	{
		PublicDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"CoreUObject",
				"Engine",
				"MovieScene",
				"MovieSceneTracks",
				"TimeManagement",
				"UMG",
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"DeveloperSettings",
				"MediaAssets",
				"UniversalObjectLocator",
			}
		);

		if (Target.bBuildEditor == true)
		{
			PrivateDependencyModuleNames.AddRange(
				new string[] {
					"ActorPickerMode",
					"EditorFramework",
					"PropertyEditor",
					"UnrealEd"
				}
			);

			PrivateIncludePathModuleNames.AddRange(
				new string[] {
					"SceneOutliner"
				}
			);
		}
	}
}
