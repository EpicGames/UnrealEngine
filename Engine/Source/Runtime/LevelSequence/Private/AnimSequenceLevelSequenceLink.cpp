// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimSequenceLevelSequenceLink.h"
#include "LevelSequence.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimSequenceLevelSequenceLink)


void UAnimSequenceLevelSequenceLink::SetLevelSequence(ULevelSequence* InLevelSequence)
{
	PathToLevelSequence = FSoftObjectPath(InLevelSequence);
}

ULevelSequence* UAnimSequenceLevelSequenceLink::ResolveLevelSequence()
{
	UObject *Object = PathToLevelSequence.TryLoad();
	return Cast<ULevelSequence>(Object);
}


UAnimSequenceLevelSequenceLink::UAnimSequenceLevelSequenceLink(class FObjectInitializer const& ObjectInitializer)
	: Super(ObjectInitializer)
{

}
