// Copyright Epic Games, Inc. All Rights Reserved.

#include "DefaultLevelSequenceInstanceData.h"

#if WITH_EDITOR
void UDefaultLevelSequenceInstanceData::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	FName PropertyName = (PropertyChangedEvent.Property != nullptr) ? PropertyChangedEvent.Property->GetFName() : NAME_None;

	if (PropertyName == GET_MEMBER_NAME_CHECKED(UDefaultLevelSequenceInstanceData, TransformOriginActor) || PropertyName == GET_MEMBER_NAME_CHECKED(UDefaultLevelSequenceInstanceData, TransformOrigin))
	{
		TransformOriginChangedEvent.Broadcast();
	}

	Super::PostEditChangeProperty(PropertyChangedEvent);
}
#endif // WITH_EDITOR