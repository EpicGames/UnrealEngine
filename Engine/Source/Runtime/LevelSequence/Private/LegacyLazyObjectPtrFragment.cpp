// Copyright Epic Games, Inc. All Rights Reserved.

#include "LegacyLazyObjectPtrFragment.h"
#include "UniversalObjectLocatorInitializeParams.h"
#include "UniversalObjectLocatorInitializeResult.h"
#include "UniversalObjectLocatorFragmentTypeHandle.h"
#include "UniversalObjectLocatorResolveParams.h"
#include "UniversalObjectLocatorStringParams.h"
#include "UObject/Package.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LegacyLazyObjectPtrFragment)

UE::UniversalObjectLocator::TFragmentTypeHandle<FLegacyLazyObjectPtrFragment> FLegacyLazyObjectPtrFragment::FragmentType;

UE::UniversalObjectLocator::FResolveResult FLegacyLazyObjectPtrFragment::Resolve(const UE::UniversalObjectLocator::FResolveParams& Params) const
{
	using namespace UE::UniversalObjectLocator;

	if (LazyObjectId.IsValid() && Params.Context != nullptr)
	{
		int32 PIEInstanceID = Params.Context->GetOutermost()->GetPIEInstanceID();
		FUniqueObjectGuid FixedUpId = PIEInstanceID == -1 ? FUniqueObjectGuid(LazyObjectId) : FUniqueObjectGuid(LazyObjectId).FixupForPIE(PIEInstanceID);

		if (PIEInstanceID != -1 && FixedUpId == LazyObjectId)
		{
			return FResolveResult();
		}

		FLazyObjectPtr LazyPtr;
		LazyPtr = FixedUpId;

		return FResolveResultData(LazyPtr.Get());
	}


	return FResolveResult();
}

UE::UniversalObjectLocator::FInitializeResult FLegacyLazyObjectPtrFragment::Initialize(const UE::UniversalObjectLocator::FInitializeParams& InParams)
{
	using namespace UE::UniversalObjectLocator;

	// This should never be called
	checkNoEntry();
	return FInitializeResult::Failure();
}

void FLegacyLazyObjectPtrFragment::ToString(FStringBuilderBase& OutStringBuilder) const
{
	// Not implemented
}

UE::UniversalObjectLocator::FParseStringResult FLegacyLazyObjectPtrFragment::TryParseString(FStringView InString, const UE::UniversalObjectLocator::FParseStringParams& Params)
{
	// Not implemented
	return UE::UniversalObjectLocator::FParseStringResult().Failure(FText());
}

uint32 FLegacyLazyObjectPtrFragment::ComputePriority(const UObject* ObjectToReference, const UObject* Context)
{
	// We can't use this at all unless explicitly added by code
	return 0;
}

