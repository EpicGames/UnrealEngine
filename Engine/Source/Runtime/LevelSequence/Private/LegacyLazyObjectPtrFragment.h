// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Misc/Guid.h"
#include "UniversalObjectLocatorFwd.h"
#include "IUniversalObjectLocatorModule.h"
#include "LegacyLazyObjectPtrFragment.generated.h"

class UObject;

USTRUCT()
struct FLegacyLazyObjectPtrFragment
{
	GENERATED_BODY()

	UPROPERTY()
	FGuid LazyObjectId;

	static UE::UniversalObjectLocator::TFragmentTypeHandle<FLegacyLazyObjectPtrFragment> FragmentType;

	friend uint32 GetTypeHash(const FLegacyLazyObjectPtrFragment& A)
	{
		return GetTypeHash(A.LazyObjectId);
	}

	friend bool operator==(const FLegacyLazyObjectPtrFragment& A, const FLegacyLazyObjectPtrFragment& B)
	{
		return A.LazyObjectId == B.LazyObjectId;
	}

	UE::UniversalObjectLocator::FResolveResult Resolve(const UE::UniversalObjectLocator::FResolveParams& Params) const;
	UE::UniversalObjectLocator::FInitializeResult Initialize(const UE::UniversalObjectLocator::FInitializeParams& InParams);
	void ToString(FStringBuilderBase& OutStringBuilder) const;
	UE::UniversalObjectLocator::FParseStringResult TryParseString(FStringView InString, const UE::UniversalObjectLocator::FParseStringParams& Params);

	static uint32 ComputePriority(const UObject* Object, const UObject* Context);
};

template<>
struct TStructOpsTypeTraits<FLegacyLazyObjectPtrFragment> : public TStructOpsTypeTraitsBase2<FLegacyLazyObjectPtrFragment>
{
	enum
	{
		WithIdenticalViaEquality = true,
	};
};
