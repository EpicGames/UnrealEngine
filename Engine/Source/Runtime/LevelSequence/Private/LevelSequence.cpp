// Copyright Epic Games, Inc. All Rights Reserved.

#include "LevelSequence.h"
#include "IMovieSceneMetaData.h"
#include "MovieSceneMetaData.h"
#include "Engine/EngineTypes.h"
#include "HAL/IConsoleManager.h"
#include "UniversalObjectLocator.h"
#include "UniversalObjectLocatorFragmentType.h"
#include "UniversalObjectLocatorResolveParameterBuffer.inl"
#include "UniversalObjectLocators/ActorLocatorFragment.h"
#include "WorldPartition/IWorldPartitionObjectResolver.h"
#include "LegacyLazyObjectPtrFragment.h"
#include "SubObjectLocator.h"
#include "Components/ActorComponent.h"
#include "Components/SkeletalMeshComponent.h"
#include "GameFramework/Actor.h"
#include "LevelSequenceDirector.h"
#include "Engine/Engine.h"
#include "MovieScene.h"
#include "MovieSceneCommonHelpers.h"
#include "UObject/AssetRegistryTagsContext.h"
#include "UObject/Package.h"
#include "UObject/UObjectHash.h"
#include "Animation/AnimInstance.h"
#include "LevelSequenceModule.h"
#include "MovieSceneSpawnableAnnotation.h"
#include "Tracks/MovieScene3DAttachTrack.h"
#include "Tracks/MovieScene3DPathTrack.h"
#include "Tracks/MovieSceneAudioTrack.h"
#include "Tracks/MovieSceneCameraCutTrack.h"
#include "Tracks/MovieSceneCinematicShotTrack.h"
#include "Tracks/MovieSceneEventTrack.h"
#include "Tracks/MovieSceneFadeTrack.h"
#include "Tracks/MovieSceneLevelVisibilityTrack.h"
#include "Tracks/MovieSceneDataLayerTrack.h"
#include "Tracks/MovieSceneMaterialParameterCollectionTrack.h"
#include "Tracks/MovieSceneSkeletalAnimationTrack.h"
#include "Tracks/MovieSceneSlomoTrack.h"
#include "Tracks/MovieSceneSpawnTrack.h"
#include "Tracks/MovieSceneSubTrack.h"
#include "Tracks/MovieSceneTimeWarpTrack.h"
#include "Tracks/MovieSceneCVarTrack.h"
#include "Tracks/MovieSceneBindingLifetimeTrack.h"
#include "Modules/ModuleManager.h"
#include "LevelSequencePlayer.h"
#include "Compilation/MovieSceneCompiledDataManager.h"
#include "Evaluation/MovieSceneEvaluationTemplateInstance.h"
#include "UniversalObjectLocators/AnimInstanceLocatorFragment.h"
#include "Engine/AssetUserData.h"
#include "Misc/App.h"
#include "Misc/DateTime.h"
#include "UObject/FortniteMainBranchObjectVersion.h"
#include "Bindings/MovieSceneSpawnableDirectorBlueprintBinding.h"
#include "Bindings/MovieSceneReplaceableDirectorBlueprintBinding.h"
#include "Bindings/MovieSceneSpawnableActorBinding.h"
#include "MovieSceneFolder.h"
#include "Sections/MovieSceneBindingLifetimeSection.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LevelSequence)


#if WITH_EDITOR
	#include "UObject/SequencerObjectVersion.h"
	#include "UObject/ObjectRedirector.h"

ULevelSequence::FPostDuplicateEvent ULevelSequence::PostDuplicateEvent;
ULevelSequence::FFixupDynamicBindingsEvent ULevelSequence::FixupDynamicBindingsEvent;

#endif

static TAutoConsoleVariable<int32> CVarDefaultLockEngineToDisplayRate(
	TEXT("LevelSequence.DefaultLockEngineToDisplayRate"),
	0,
	TEXT("0: Playback locked to playback frames\n1: Unlocked playback with sub frame interpolation"),
	ECVF_Default);

static TAutoConsoleVariable<FString> CVarDefaultTickResolution(
	TEXT("LevelSequence.DefaultTickResolution"),
	TEXT("24000fps"),
	TEXT("Specifies the default tick resolution for newly created level sequences. Examples: 30 fps, 120/1 (120 fps), 30000/1001 (29.97), 0.01s (10ms)."),
	ECVF_Default);

static TAutoConsoleVariable<FString> CVarDefaultDisplayRate(
	TEXT("LevelSequence.DefaultDisplayRate"),
	TEXT("30fps"),
	TEXT("Specifies the default display frame rate for newly created level sequences; also defines frame locked frame rate where sequences are set to be frame locked. Examples: 30 fps, 120/1 (120 fps), 30000/1001 (29.97), 0.01s (10ms)."),
	ECVF_Default);

static TAutoConsoleVariable<int32> CVarDefaultClockSource(
	TEXT("LevelSequence.DefaultClockSource"),
	0,
	TEXT("Specifies the default clock source for newly created level sequences. 0: Tick, 1: Platform, 2: Audio, 3: RelativeTimecode, 4: Timecode, 5: Custom"),
	ECVF_Default);

ULevelSequence::ULevelSequence(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, MovieScene(nullptr)
{
	bParentContextsAreSignificant = true;
}

void ULevelSequence::Initialize()
{
	MovieScene = NewObject<UMovieScene>(this, NAME_None, RF_Transactional);

	const bool bFrameLocked = CVarDefaultLockEngineToDisplayRate.GetValueOnGameThread() != 0;

	MovieScene->SetEvaluationType( bFrameLocked ? EMovieSceneEvaluationType::FrameLocked : EMovieSceneEvaluationType::WithSubFrames );

	FFrameRate TickResolution(60000, 1);
	TryParseString(TickResolution, *CVarDefaultTickResolution.GetValueOnGameThread());
	MovieScene->SetTickResolutionDirectly(TickResolution);

	FFrameRate DisplayRate(30, 1);
	TryParseString(DisplayRate, *CVarDefaultDisplayRate.GetValueOnGameThread());
	MovieScene->SetDisplayRate(DisplayRate);

	int32 ClockSource = CVarDefaultClockSource.GetValueOnGameThread();
	MovieScene->SetClockSource((EUpdateClockSource)ClockSource);

#if WITH_EDITOR
	UMovieSceneMetaData* MetaData = FindOrAddMetaData<UMovieSceneMetaData>();
	MetaData->SetCreated(FDateTime::UtcNow());
	// Only set Author if it has not been set already, potentially by a creation wizard.
	if (MetaData->GetAuthor().IsEmpty())
	{
		MetaData->SetAuthor(FApp::GetSessionOwner());
	}
#endif
}

UObject* ULevelSequence::MakeSpawnableTemplateFromInstance(UObject& InSourceObject, FName ObjectName)
{
	return MovieSceneHelpers::MakeSpawnableTemplateFromInstance(InSourceObject, MovieScene, ObjectName);
}

bool ULevelSequence::CanAnimateObject(UObject& InObject) const 
{
	return InObject.IsA<AActor>() || InObject.IsA<UActorComponent>() || InObject.IsA<UAnimInstance>();
}

#if WITH_EDITOR

ETrackSupport ULevelSequence::IsTrackSupportedImpl(TSubclassOf<class UMovieSceneTrack> InTrackClass) const
{
	if (InTrackClass == UMovieScene3DAttachTrack::StaticClass() ||
		InTrackClass == UMovieScene3DPathTrack::StaticClass() ||
		InTrackClass == UMovieSceneAudioTrack::StaticClass() ||
		InTrackClass == UMovieSceneCameraCutTrack::StaticClass() ||
		InTrackClass == UMovieSceneCinematicShotTrack::StaticClass() ||
		InTrackClass == UMovieSceneEventTrack::StaticClass() ||
		InTrackClass == UMovieSceneFadeTrack::StaticClass() ||
		InTrackClass == UMovieSceneLevelVisibilityTrack::StaticClass() ||
		InTrackClass == UMovieSceneDataLayerTrack::StaticClass() ||
		InTrackClass == UMovieSceneMaterialParameterCollectionTrack::StaticClass() ||
		InTrackClass == UMovieSceneSkeletalAnimationTrack::StaticClass() ||
		InTrackClass == UMovieSceneSlomoTrack::StaticClass() ||
		InTrackClass == UMovieSceneSpawnTrack::StaticClass() ||
		InTrackClass == UMovieSceneSubTrack::StaticClass() ||
		InTrackClass == UMovieSceneCVarTrack::StaticClass() ||
		InTrackClass == UMovieSceneBindingLifetimeTrack::StaticClass() ||
		InTrackClass == UMovieSceneTimeWarpTrack::StaticClass())
	{
		return ETrackSupport::Supported;
	}

	return Super::IsTrackSupportedImpl(InTrackClass);
}

bool ULevelSequence::IsFilterSupportedImpl(const FString& InFilterName) const
{
	static const TArray<FString> SupportedFilters = {
		TEXT("Audio"),
		TEXT("Camera"),
		TEXT("CameraCut"),
		TEXT("CinematicShot"),
		TEXT("Condition"),
		TEXT("ControlRigControl"),
		TEXT("DataLayer"),
		TEXT("Event"),
		TEXT("Fade"),
		TEXT("Folder"),
		TEXT("Keyed"),
		TEXT("Group"),
		TEXT("Level"),
		TEXT("LevelVisibility"),
		TEXT("Light"),
		TEXT("LiveLink"),
		TEXT("Media"),
		TEXT("Niagara"),
		TEXT("Particle"),
		TEXT("Selected"),
		TEXT("SelectedAnimLayersFilter"),
		TEXT("SelectedControlRigControl"),
		TEXT("SkeletalMesh"),
		TEXT("StickySelection"),
		TEXT("Subsequence"),
		TEXT("TimeDilation"),
		TEXT("TimeWarp"),
		TEXT("Unbound")
	};
	return SupportedFilters.Contains(InFilterName);
}

void ULevelSequence::GetAssetRegistryTags(TArray<FAssetRegistryTag>& OutTags) const
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS;
	Super::GetAssetRegistryTags(OutTags);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS;
}

void ULevelSequence::GetAssetRegistryTags(FAssetRegistryTagsContext Context) const
{
#if WITH_EDITORONLY_DATA
	if (DirectorBlueprint)
	{
		DirectorBlueprint->GetAssetRegistryTags(Context);
	}
#endif

	for (UObject* MetaData : MetaDataObjects)
	{
		IMovieSceneMetaDataInterface* MetaDataInterface = Cast<IMovieSceneMetaDataInterface>(MetaData);
		if (MetaDataInterface)
		{
			PRAGMA_DISABLE_DEPRECATION_WARNINGS;
			TArray<UObject::FAssetRegistryTag> DeprecatedFunctionTags;
			MetaDataInterface->ExtendAssetRegistryTags(DeprecatedFunctionTags);
			for (UObject::FAssetRegistryTag& Tag : DeprecatedFunctionTags)
			{
				Context.AddTag(MoveTemp(Tag));
			}
			PRAGMA_ENABLE_DEPRECATION_WARNINGS;
			MetaDataInterface->ExtendAssetRegistryTags(Context);
		}
	}

	Super::GetAssetRegistryTags(Context);
}

void ULevelSequence::GetAssetRegistryTagMetadata(TMap<FName, FAssetRegistryTagMetadata>& OutMetadata) const
{
	for (UObject* MetaData : MetaDataObjects)
	{
		IMovieSceneMetaDataInterface* MetaDataInterface = Cast<IMovieSceneMetaDataInterface>(MetaData);
		if (MetaDataInterface)
		{
			MetaDataInterface->ExtendAssetRegistryTagMetaData(OutMetadata);
		}
	}

	Super::GetAssetRegistryTagMetadata(OutMetadata);
}

void ULevelSequence::ThreadedPostLoadAssetRegistryTagsOverride(FPostLoadAssetRegistryTagsContext& Context) const
{
	Super::ThreadedPostLoadAssetRegistryTagsOverride(Context);

	// GetAssetRegistryTags appends the DirectorBlueprint tags to the World's tags, so we also have to run the Blueprint ThreadedPostLoadAssetRegistryTagsOverride
	UBlueprint::PostLoadBlueprintAssetRegistryTags(Context);
}

void PurgeLegacyBlueprints(UObject* InObject, UPackage* Package)
{
	if (UBlueprint* BP = Cast<UBlueprint>(InObject))
	{
		UPackage* TransientPackage = GetTransientPackage();

		{
			FString OldName = BP->GetName();

			BP->ClearFlags(RF_Public);
			BP->SetFlags(RF_Transient);
			BP->RemoveFromRoot();

			FName NewName = MakeUniqueObjectName(TransientPackage, UBlueprint::StaticClass(), *FString::Printf(TEXT("DEAD_SPAWNABLE_BLUEPRINT_%s"), *BP->GetName()));
			BP->Rename(*NewName.ToString(), GetTransientPackage(), REN_NonTransactional | REN_DoNotDirty);

			UE_LOGF(LogLevelSequence, Log, "Discarding blueprint '%ls' from package '%ls'.", *OldName, *Package->GetName());
		}

		if (BP->GeneratedClass)
		{
			FName    OldName    = BP->GeneratedClass->GetFName();
			UObject* OldOuter   = BP->GeneratedClass->GetOuter();
			UClass*  SuperClass = BP->GeneratedClass->GetSuperClass();

			if (UObject* ClassDefaultObject = BP->GeneratedClass->GetDefaultObject(false))
			{
				ClassDefaultObject->ClearFlags(RF_Public);
				ClassDefaultObject->SetFlags(RF_Transient);
				ClassDefaultObject->RemoveFromRoot();
			}

			BP->GeneratedClass->ClearFlags(RF_Public);
			BP->GeneratedClass->SetFlags(RF_Transient);
			BP->GeneratedClass->ClassFlags |= CLASS_Deprecated;
			BP->GeneratedClass->RemoveFromRoot();

			FName NewName = MakeUniqueObjectName(TransientPackage, BP->GeneratedClass, *FString::Printf(TEXT("DEAD_SPAWNABLE_BP_CLASS_%s_C"), *BP->GeneratedClass->ClassGeneratedBy->GetName()));
			BP->GeneratedClass->Rename(*NewName.ToString(), GetTransientPackage(), REN_DoNotDirty | REN_NonTransactional);

			if (SuperClass)
			{
				UObjectRedirector* Redirector = NewObject<UObjectRedirector>(OldOuter, OldName);
				Redirector->DestinationObject = SuperClass;

				UE_LOGF(LogLevelSequence, Log, "Discarding generated class '%ls' from package '%ls'. Replacing with redirector to '%ls'", *OldName.ToString(), *Package->GetName(), *SuperClass->GetName());
			}
			else
			{
				UE_LOGF(LogLevelSequence, Log, "Discarding generated class '%ls' from package '%ls'. Unable to create redirector due to no super class.", *OldName.ToString(), *Package->GetName());
			}
		}
	}
}
#endif

void ULevelSequence::PostDuplicate(bool bDuplicateForPIE)
{
	Super::PostDuplicate(bDuplicateForPIE);

#if WITH_EDITORONLY_DATA
	if (DirectorBlueprint)
	{
		DirectorClass = DirectorBlueprint->GeneratedClass.Get();

		// Remove the binding for the director blueprint recompilation and re-add it to be sure there is only one entry in the list
		DirectorBlueprint->OnCompiled().RemoveAll(this);
		DirectorBlueprint->OnCompiled().AddUObject(this, &ULevelSequence::OnDirectorRecompiled);
	}
	else
	{
		DirectorClass = nullptr;
	}
#endif

#if WITH_EDITOR
	if (PostDuplicateEvent.IsBound())
	{
		PostDuplicateEvent.Execute(this);
	}
#endif

#if WITH_EDITOR
	UMovieSceneMetaData* MetaData = FindOrAddMetaData<UMovieSceneMetaData>();
	MetaData->SetCreated(FDateTime::UtcNow());
	MetaData->SetAuthor(FApp::GetSessionOwner());
	MetaData->SetNotes(FString()); // Intentionally clear the notes
#endif
}

void ULevelSequence::PostLoad()
{
	Super::PostLoad();

#if WITH_EDITOR
	if (MovieScene)
	{
		// Remove any invalid object bindings. This was moved from PostInitProperties
		//   because it has to happen after the asset has actually been serialized.
		TSet<FGuid> ValidObjectBindings;
		for (int32 Index = 0; Index < MovieScene->GetSpawnableCount(); ++Index)
		{
			ValidObjectBindings.Add(MovieScene->GetSpawnable(Index).GetGuid());
		}
		for (int32 Index = 0; Index < MovieScene->GetPossessableCount(); ++Index)
		{
			ValidObjectBindings.Add(MovieScene->GetPossessable(Index).GetGuid());
		}

		BindingReferences.RemoveInvalidBindings(ValidObjectBindings);
	}

	if (!DirectorBlueprint)
	{
		UBlueprint* PhantomDirector = FindObject<UBlueprint>(this, TEXT("SequenceDirector"));
		if (!ensureMsgf(!PhantomDirector, TEXT("Phantom sequence director found in sequence '%s' which has a nullptr DirectorBlueprint. Re-assigning to prevent future crash."), *GetName()))
		{
			DirectorBlueprint = PhantomDirector;
		}
	}

	if (DirectorBlueprint)
	{
		DirectorBlueprint->ClearFlags(RF_Standalone);

		// Remove the binding for the director blueprint recompilation and re-add it to be sure there is only one entry in the list
		DirectorBlueprint->OnCompiled().RemoveAll(this);
		DirectorBlueprint->OnCompiled().AddUObject(this, &ULevelSequence::OnDirectorRecompiled);

		const int32 RenameFlags = REN_AllowPackageLinkerMismatch | REN_NonTransactional | REN_DoNotDirty | REN_DontCreateRedirectors;
		if (DirectorBlueprint->Rename(*GetDirectorBlueprintName(), nullptr, (RenameFlags | REN_Test)))
		{
			DirectorBlueprint->Rename(*GetDirectorBlueprintName(), nullptr, RenameFlags);
		}
	}

	if (MovieScene)
	{
		TSet<FGuid> InvalidSpawnables;

		for (int32 Index = 0; Index < MovieScene->GetSpawnableCount(); ++Index)
		{
			FMovieSceneSpawnable& Spawnable = MovieScene->GetSpawnable(Index);
			if (!Spawnable.GetObjectTemplate())
			{
				if (Spawnable.GeneratedClass_DEPRECATED && Spawnable.GeneratedClass_DEPRECATED->ClassGeneratedBy)
				{
					const FName TemplateName = MakeUniqueObjectName(MovieScene, UObject::StaticClass(), Spawnable.GeneratedClass_DEPRECATED->ClassGeneratedBy->GetFName());

					UObject* NewTemplate = NewObject<UObject>(MovieScene, Spawnable.GeneratedClass_DEPRECATED->GetSuperClass(), TemplateName);
					if (NewTemplate)
					{
						Spawnable.CopyObjectTemplate(*NewTemplate, *this);
					}
				}
			}

			if (!Spawnable.GetObjectTemplate())
			{
				InvalidSpawnables.Add(Spawnable.GetGuid());
				UE_LOGF(LogLevelSequence, Warning, "Spawnable '%ls' with ID '%ls' does not have a valid object template", *Spawnable.GetName(), *Spawnable.GetGuid().ToString());
			}
		}
	}

	if (GetLinkerCustomVersion(FSequencerObjectVersion::GUID) < FSequencerObjectVersion::PurgeSpawnableBlueprints)
	{
		// Remove any old generated classes from the package that will have been left behind from when we used blueprints for spawnables		
		{
			UPackage* Package = GetOutermost();
			TArray<UObject*> PackageSubobjects;
			GetObjectsWithOuter(Package, PackageSubobjects, EGetObjectsFlags::None);
			for (UObject* ObjectInPackage : PackageSubobjects)
			{
				PurgeLegacyBlueprints(ObjectInPackage, Package);
			}
		}
	}

	for (TPair<FGuid, FLevelSequenceLegacyObjectReference>& Pair : ObjectReferences_DEPRECATED.Map)
	{
		if (Pair.Value.ObjectId.IsValid())
		{
			FUniversalObjectLocator NewLocator;
			NewLocator.AddFragment<FLegacyLazyObjectPtrFragment>(Pair.Value.ObjectId.GetGuid());
			BindingReferences.FMovieSceneBindingReferences::AddBinding(Pair.Key, MoveTemp(NewLocator));
		}
		else if (Pair.Value.ObjectPath.Len() > 0)
		{
			FUniversalObjectLocator NewLocator;
			NewLocator.AddFragment<FSubObjectLocator>(Pair.Value.ObjectPath);
			BindingReferences.FMovieSceneBindingReferences::AddBinding(Pair.Key, MoveTemp(NewLocator));
		}
	}
	ObjectReferences_DEPRECATED.Map.Empty();

	if (GetLinkerCustomVersion(FFortniteMainBranchObjectVersion::GUID) < FFortniteMainBranchObjectVersion::LevelSequenceUpgradeDynamicBindings)
	{
		bool bConvertedDynamicBinding = ConvertOldSpawnables();

		for (int32 Index = 0; Index < MovieScene->GetPossessableCount(); ++Index)
		{
			FMovieScenePossessable& Possessable = MovieScene->GetPossessable(Index);
			if (Possessable.DynamicBinding_DEPRECATED.Function)
			{
				bConvertedDynamicBinding = true;
				ConvertDynamicBindingPossessable(Possessable);
			}
		}
		if (bConvertedDynamicBinding && FixupDynamicBindingsEvent.IsBound())
		{
			if (!GetPackage()->GetHasBeenEndLoaded())
			{
				FCoreUObjectDelegates::OnEndLoadPackage.AddUObject(this, &ULevelSequence::OnEndLoadPackage);
			}
			else
			{
				FixupDynamicBindingsEvent.Broadcast(this);
			}
		}
	}

#endif
}

#if WITH_EDITOR
void ULevelSequence::OnEndLoadPackage(const FEndLoadPackageContext& Context)
{
	FCoreUObjectDelegates::OnEndLoadPackage.RemoveAll(this);

	if (FixupDynamicBindingsEvent.IsBound())
	{
		FixupDynamicBindingsEvent.Broadcast(this);
	}
}


bool ULevelSequence::ConvertOldSpawnables()
{
	bool bConvertedDynamicBinding = false;
	while (MovieScene->GetSpawnableCount() > 0)
	{
		FMovieSceneSpawnable& Spawnable = MovieScene->GetSpawnable(0);

		FMovieScenePossessable* CreatedPossessable = nullptr;

		UObject* ObjectToConvert = Spawnable.GetObjectTemplate();

		UClass* CustomBindingType = nullptr;
		if (Spawnable.DynamicBinding_DEPRECATED.Function)
		{
			CustomBindingType = UMovieSceneSpawnableDirectorBlueprintBinding::StaticClass();
		}
		else
		{
			// Search through custom binding types to find one that best supports the template type
			static TArray<const TSubclassOf<UMovieSceneCustomBinding>> CachedCustomBindingTypes;
			static bool CustomBindingTypesCached = false;
			if (!CustomBindingTypesCached)
			{
				CustomBindingTypesCached = true;
				MovieSceneHelpers::GetPrioritySortedCustomBindingTypes(CachedCustomBindingTypes);
			}

			for (const TSubclassOf<UMovieSceneCustomBinding>& CandidateCustomBindingType : CachedCustomBindingTypes)
			{
				if (CandidateCustomBindingType && CandidateCustomBindingType->IsChildOf(UMovieSceneSpawnableBindingBase::StaticClass()) && CandidateCustomBindingType->GetDefaultObject<UMovieSceneCustomBinding>()->SupportsBindingCreationFromObject(Spawnable.GetObjectTemplate()))
				{
					CustomBindingType = CandidateCustomBindingType;
					break;
				}
			}
		}

		if (!CustomBindingType)
		{
			UE_LOGF(LogLevelSequence, Warning, "Could not upgrade Spawnable '%ls' with ID '%ls'", *Spawnable.GetName(), *Spawnable.GetGuid().ToString());
			break;
		}

		UMovieSceneCustomBinding* NewCustomBinding = CustomBindingType->GetDefaultObject<UMovieSceneCustomBinding>()->CreateNewCustomBinding(ObjectToConvert, *MovieScene);

		if (!NewCustomBinding)
		{
			UE_LOGF(LogLevelSequence, Warning, "Could not upgrade Spawnable '%ls' with ID '%ls'", *Spawnable.GetName(), *Spawnable.GetGuid().ToString());
			break;
		}

		if (UMovieSceneSpawnableDirectorBlueprintBinding* DirectorBlueprintBinding = Cast<UMovieSceneSpawnableDirectorBlueprintBinding>(NewCustomBinding))
		{
			// Copy over the binding info
			DirectorBlueprintBinding->DynamicBinding = Spawnable.DynamicBinding_DEPRECATED;
			bConvertedDynamicBinding = true;
		}
		else if (UMovieSceneSpawnableActorBinding* SpawnableActorBinding = Cast<UMovieSceneSpawnableActorBinding>(NewCustomBinding))
		{
			SpawnableActorBinding->bNetAddressableName = Spawnable.bNetAddressableName;
			SpawnableActorBinding->LevelName = Spawnable.LevelName;
			SpawnableActorBinding->bContinuouslyRespawn = Spawnable.bContinuouslyRespawn;
			SpawnableActorBinding->SpawnOwnership = Spawnable.GetSpawnOwnership();
		}

		FString PossessableName = Spawnable.GetName();
		FGuid SpawnableGuid = Spawnable.GetGuid();

		FMovieScenePossessable NewPossessable(PossessableName, NewCustomBinding->GetBoundObjectClass());
		// Steal guid
		NewPossessable.SetGuid(SpawnableGuid);

		if (FMovieSceneBinding* SpawnableBinding = MovieScene->FindBinding(SpawnableGuid))
		{
			// Copy binding and track references to be tied to the new possessable
			FMovieSceneBinding PossessableBinding = *SpawnableBinding;
			// Add the custom binding. We use the spawnable binding here since it won't have a binding reference yet, and we need to steal the id
			BindingReferences.AddOrReplaceBinding(SpawnableGuid, NewCustomBinding, 0);

			// Remove the spawnable and all its' sub tracks
			if (MovieScene->RemoveSpawnable(SpawnableGuid))
			{
				// Add the new possessable with the copied binding
				MovieScene->AddPossessable(NewPossessable, PossessableBinding);
			}
		}
	}
	return bConvertedDynamicBinding;
}

void ULevelSequence::ConvertDynamicBindingPossessable(FMovieScenePossessable& Possessable)
{
	UMovieSceneReplaceableDirectorBlueprintBinding* NewCustomBinding = nullptr;
	
	const FName InstancedBindingName = MakeUniqueObjectName(MovieScene, UObject::StaticClass(), *FString(Possessable.GetName() + TEXT("_CustomBinding")));
	NewCustomBinding = NewObject<UMovieSceneReplaceableDirectorBlueprintBinding>(MovieScene, UMovieSceneReplaceableDirectorBlueprintBinding::StaticClass(), InstancedBindingName, RF_Transactional);
	
	if (!NewCustomBinding)
	{
		return;
	}

	if (UMovieSceneReplaceableDirectorBlueprintBinding* DirectorBlueprintBinding = Cast<UMovieSceneReplaceableDirectorBlueprintBinding>(NewCustomBinding))
	{
		// Copy over the binding info
		DirectorBlueprintBinding->DynamicBinding = Possessable.DynamicBinding_DEPRECATED;
		Possessable.DynamicBinding_DEPRECATED = FMovieSceneDynamicBinding();
	}

	// Replace the current binding with the new one. We call RemoveBinding first because if there were multiple bindings for this track,
	// they would have been overridden with the Dynamic Binding anyway, and so we ensure that stays the same by keeping only one binding
	BindingReferences.RemoveBinding(Possessable.GetGuid());
	BindingReferences.AddOrReplaceBinding(Possessable.GetGuid(), NewCustomBinding, 0);

	// Add a binding lifetime track if not present
	UMovieSceneBindingLifetimeTrack* BindingLifetimeTrack = Cast<UMovieSceneBindingLifetimeTrack>(MovieScene->FindTrack(UMovieSceneBindingLifetimeTrack::StaticClass(), Possessable.GetGuid(), NAME_None));
	if (!BindingLifetimeTrack)
	{
		BindingLifetimeTrack = Cast<UMovieSceneBindingLifetimeTrack>(MovieScene->AddTrack(UMovieSceneBindingLifetimeTrack::StaticClass(), Possessable.GetGuid()));
	}

	if (BindingLifetimeTrack && BindingLifetimeTrack->GetAllSections().IsEmpty())
	{
		UMovieSceneBindingLifetimeSection* BindingLifetimeSection = Cast<UMovieSceneBindingLifetimeSection>(BindingLifetimeTrack->CreateNewSection());
		BindingLifetimeSection->SetRange(TRange<FFrameNumber>::All());
		BindingLifetimeTrack->AddSection(*BindingLifetimeSection);
	}
}

#endif

#if WITH_EDITORONLY_DATA
void ULevelSequence::DeclareConstructClasses(TArray<FTopLevelAssetPath>& OutConstructClasses, const UClass* SpecificSubclass)
{
	Super::DeclareConstructClasses(OutConstructClasses, SpecificSubclass);
	OutConstructClasses.Add(FTopLevelAssetPath(UObjectRedirector::StaticClass()));
	OutConstructClasses.Add(FTopLevelAssetPath(UMovieSceneSpawnableActorBinding::StaticClass()));
	OutConstructClasses.Add(FTopLevelAssetPath(UMovieSceneSpawnableDirectorBlueprintBinding::StaticClass()));
}
#endif

void ULevelSequence::PostInitProperties()
{
	Super::PostInitProperties();
}

bool ULevelSequence::Rename(const TCHAR* NewName, UObject* NewOuter, ERenameFlags Flags)
{
	bool bRetVal = Super::Rename(NewName, NewOuter, Flags);

#if WITH_EDITOR
	if (DirectorBlueprint)
	{
		DirectorBlueprint->Rename(*GetDirectorBlueprintName(), this, Flags);
	}
#endif

	return bRetVal;
}

void ULevelSequence::BindPossessableObject(const FGuid& ObjectId, UObject& PossessedObject, UObject* Context)
{
	if (Context)
	{
		BindingReferences.AddBinding(ObjectId, &PossessedObject, Context);
	}
}

bool ULevelSequence::CanPossessObject(UObject& Object, UObject* InPlaybackContext) const
{
	return true;
}

void ULevelSequence::LocateBoundObjects(const FGuid& ObjectId, UObject* Context, const FLevelSequenceBindingReference::FResolveBindingParams& InResolveBindingParams, TArray<UObject*, TInlineAllocator<1>>& OutObjects) const
{
	using namespace UE::UniversalObjectLocator;

	TResolveParamsWithBuffer<128> ResolveParams;

	ResolveParams.AddParameter(FActorLocatorFragmentResolveParameter::ParameterType,
		InResolveBindingParams.StreamingWorld,
		InResolveBindingParams.WorldPartitionResolveData ? InResolveBindingParams.WorldPartitionResolveData->ContainerID : FActorContainerID(),
		InResolveBindingParams.WorldPartitionResolveData ? InResolveBindingParams.WorldPartitionResolveData->SourceWorldAssetPath : InResolveBindingParams.StreamedLevelAssetPath
		);

	LocateBoundObjects(ObjectId, ResolveParams, nullptr, OutObjects);
}

FGuid ULevelSequence::FindBindingFromObject(UObject* InObject, UObject* Context) const
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	return UMovieSceneSequence::FindBindingFromObject(InObject, Context);
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}


FGuid ULevelSequence::FindBindingFromObject(UObject* InObject, TSharedRef<const UE::MovieScene::FSharedPlaybackState> SharedPlaybackState) const
{
	if (InObject)
	{
		if (FMovieSceneEvaluationState* EvaluationState = SharedPlaybackState->FindCapability<FMovieSceneEvaluationState>())
		{
			FMovieSceneSequenceID SequenceID = EvaluationState->FindSequenceId(this);
			return EvaluationState->FindCachedObjectId(*InObject, SequenceID, SharedPlaybackState);
		}
	}
	return FGuid();
}

void ULevelSequence::GatherExpiredObjects(const FMovieSceneObjectCache& InObjectCache, TArray<FGuid>& OutInvalidIDs) const
{
	using namespace UE::UniversalObjectLocator;

	TArrayView<const FMovieSceneBindingReference> References = BindingReferences.GetAllReferences();
	for (int32 Index = 0; Index < References.Num(); ++Index)
	{
		const FMovieSceneBindingReference& Reference = References[Index];
		
		if (Reference.Locator.GetLastFragmentTypeHandle() == FAnimInstanceLocatorFragment::FragmentType)
		{
			// Get anim instance fragment to determine type.
			EAnimInstanceLocatorFragmentType LocatorType = EAnimInstanceLocatorFragmentType::AnimInstance;
			if (const FAnimInstanceLocatorFragment* AnimFragment = Reference.Locator.GetLastFragment()->GetPayloadAs(FAnimInstanceLocatorFragment::FragmentType))
			{
				LocatorType = AnimFragment->Type;
			}
			
			for (TWeakObjectPtr<> WeakObject : InObjectCache.IterateBoundObjects(Reference.ID))
			{
				UAnimInstance* AnimInstance = Cast<UAnimInstance>(WeakObject.Get());
				UAnimInstance* AnimInstanceToTest = nullptr;
				if (AnimInstance && AnimInstance->GetOwningComponent())
				{
					//Check if we're a regular or post-process anim instance and if the owning component's anim instance has changed.
					AnimInstanceToTest = LocatorType == EAnimInstanceLocatorFragmentType::PostProcessAnimInstance ?
						AnimInstance->GetOwningComponent()->GetPostProcessInstance() :
						AnimInstance->GetOwningComponent()->GetAnimInstance();
				}
				if (!AnimInstance || !AnimInstance->GetOwningComponent() || AnimInstanceToTest != AnimInstance)
				{
					OutInvalidIDs.Add(Reference.ID);
				}
			}

			// Skip over subsequent matched IDs
			while (Index < References.Num()-1 && References[Index+1].ID == Reference.ID)
			{
				++Index;
			}
		}
	}
}

UMovieScene* ULevelSequence::GetMovieScene() const
{
	return MovieScene;
}

UObject* ULevelSequence::GetParentObject(UObject* Object) const
{
	if (UActorComponent* Component = Cast<UActorComponent>(Object))
	{
		return Component->GetOwner();
	}

	if (UAnimInstance* AnimInstance = Cast<UAnimInstance>(Object))
	{
		if (AnimInstance->GetWorld())
		{
			return AnimInstance->GetOwningComponent();
		}
	}

	return nullptr;
}

bool ULevelSequence::AllowsSpawnableObjects() const
{
	TArray<const TSubclassOf<UMovieSceneCustomBinding>> CustomBindingTypes;

	MovieSceneHelpers::GetPrioritySortedCustomBindingTypes(CustomBindingTypes);
	for (const TSubclassOf<UMovieSceneCustomBinding>& CustomBindingType : CustomBindingTypes)
	{
		const bool bIsCustomSpawnableBinding = CustomBindingType->IsChildOf<UMovieSceneSpawnableBindingBase>();
		if (bIsCustomSpawnableBinding)
		{
			return true;
		}
	}
	return false;
}

bool ULevelSequence::AllowsCustomBindings() const
{
	return true;
}

bool ULevelSequence::CanRebindPossessable(const FMovieScenePossessable& InPossessable) const
{
	return !InPossessable.GetParent().IsValid();
}

void ULevelSequence::UnbindPossessableObjects(const FGuid& ObjectId)
{
	BindingReferences.RemoveBinding(ObjectId);
}

void ULevelSequence::UnbindObjects(const FGuid& ObjectId, const TArray<UObject*>& InObjects, UObject* InContext)
{
	BindingReferences.RemoveObjects(ObjectId, InObjects, InContext);
}

void ULevelSequence::UnbindInvalidObjects(const FGuid& ObjectId, UObject* InContext)
{
	BindingReferences.RemoveInvalidObjects(ObjectId, InContext);
}

const FMovieSceneBindingReferences* ULevelSequence::GetBindingReferences() const
{
	return &BindingReferences;
}

#if WITH_EDITOR

UBlueprint* ULevelSequence::GetDirectorBlueprint() const
{
	return DirectorBlueprint;
}

FString ULevelSequence::GetDirectorBlueprintName() const
{
	return GetDisplayName().ToString() + "_DirectorBP";
}

void ULevelSequence::SetDirectorBlueprint(UBlueprint* NewDirectorBlueprint)
{
	if (DirectorBlueprint)
	{
		DirectorBlueprint->OnCompiled().RemoveAll(this);
	}

	DirectorBlueprint = NewDirectorBlueprint;

	if (DirectorBlueprint)
	{
		DirectorClass = NewDirectorBlueprint->GeneratedClass.Get();
		DirectorBlueprint->OnCompiled().AddUObject(this, &ULevelSequence::OnDirectorRecompiled);
	}
	else
	{
		DirectorClass = nullptr;
	}

	MarkAsChanged();
}

void ULevelSequence::OnDirectorRecompiled(UBlueprint* InCompiledBlueprint)
{
	ensure(InCompiledBlueprint == DirectorBlueprint);
	DirectorClass = DirectorBlueprint->GeneratedClass.Get();

	MarkAsChanged();
}

FGuid ULevelSequence::FindOrAddBinding(UObject* InObject)
{
	using namespace UE::MovieScene;

	UObject* PlaybackContext = InObject ? InObject->GetWorld() : nullptr;
	if (!InObject || !PlaybackContext)
	{
		return FGuid();
	}

	AActor* Actor = Cast<AActor>(InObject);
	if (Actor && Actor->ActorHasTag("SequencerActor"))
	{
		TOptional<FMovieSceneSpawnableAnnotation> Annotation = FMovieSceneSpawnableAnnotation::Find(Actor);
		if (Annotation.IsSet() && Annotation->OriginatingSequence == this)
		{
			return Annotation->ObjectBindingID;
		}
		
		// If this actor is a spawnable and is not in the same originating sequence, it's likely a spawnable that will be possessed. 
		// SetSpawnableObjectBindingID will need to be called on that possessable.
	}

	UObject* ParentObject = GetParentObject(InObject);
	FGuid    ParentGuid   = ParentObject ? FindOrAddBinding(ParentObject) : FGuid();

	if (ParentObject && !ParentGuid.IsValid())
	{
		UE_LOGF(LogLevelSequence, Error, "Unable to possess object '%ls' because it's parent could not be bound.", *InObject->GetName());
		return FGuid();
	}

	// Perform a potentially slow lookup of every possessable binding in the sequence to see if we already have this
	{
		FSharedPlaybackStateCreateParams CreateParams;
		CreateParams.PlaybackContext = PlaybackContext;
		TSharedRef<FSharedPlaybackState> TransientPlaybackState = MakeShared<FSharedPlaybackState>(*this, CreateParams);

		FMovieSceneEvaluationState State;
		TransientPlaybackState->AddCapabilityRaw(&State);
		State.AssignSequence(MovieSceneSequenceID::Root, *this, TransientPlaybackState);

		FGuid ExistingID = State.FindObjectId(*InObject, MovieSceneSequenceID::Root, TransientPlaybackState);
		if (ExistingID.IsValid())
		{
			return ExistingID;
		}
	}

	// We have to possess this object
	if (!CanPossessObject(*InObject, PlaybackContext))
	{
		return FGuid();
	}

	FString NewName = Actor ? Actor->GetActorLabel() : InObject->GetName();

	const FGuid NewGuid = MovieScene->AddPossessable(NewName, InObject->GetClass());

	// Attempt to use the parent as a context if necessary
	UObject* BindingContext = ParentObject && AreParentContextsSignificant() ? ParentObject : PlaybackContext;

	// Set up parent/child guids for possessables within spawnables
	if (ParentGuid.IsValid())
	{
		FMovieScenePossessable* ChildPossessable = MovieScene->FindPossessable(NewGuid);
		if (ensure(ChildPossessable))
		{
			ChildPossessable->SetParent(ParentGuid, MovieScene);
		}
	}

	BindPossessableObject(NewGuid, *InObject, BindingContext);

	return NewGuid;

}

FGuid ULevelSequence::CreatePossessable(UObject* ObjectToPossess)
{
	return FindOrAddBinding(ObjectToPossess);
}

FGuid ULevelSequence::CreateSpawnable(UObject* ObjectToSpawn)
{
	if (!MovieScene || !ObjectToSpawn)
	{
		return FGuid();
	}

	FGuid NewGuid = MovieSceneHelpers::TryCreateCustomSpawnableBinding(this, ObjectToSpawn);

	UMovieSceneSpawnTrack* NewSpawnTrack = MovieScene->AddTrack<UMovieSceneSpawnTrack>(NewGuid);
	if (NewSpawnTrack)
	{
		NewSpawnTrack->Modify();

		NewSpawnTrack->AddSection(*NewSpawnTrack->CreateNewSection());
	}
	return NewGuid;
}

#endif // WITH_EDITOR

UObject* ULevelSequence::CreateDirectorInstance(TSharedRef<const FSharedPlaybackState> SharedPlaybackState, FMovieSceneSequenceID SequenceID)
{
	UObject* DirectorOuter = SharedPlaybackState->GetPlaybackContext();
	IMovieScenePlayer* OptionalPlayer = UE::MovieScene::FPlayerIndexPlaybackCapability::GetPlayer(SharedPlaybackState);

#if WITH_EDITOR
	if (!UMovieScene::IsTrackClassAllowed(ULevelSequenceDirector::StaticClass()))
	{
		return nullptr;
	}
#endif

	if (DirectorClass && DirectorOuter && DirectorClass->IsChildOf(ULevelSequenceDirector::StaticClass()))
	{
		FName DirectorName = NAME_None;

#if WITH_EDITOR
		// Give it a pretty name so it shows up in the debug instances drop down nicely
		DirectorName = MakeUniqueObjectName(DirectorOuter, DirectorClass, *(GetFName().ToString() + TEXT("_Director")));
#endif

		ULevelSequencePlayer* LevelSequencePlayer = nullptr;
		if (OptionalPlayer)
		{
			LevelSequencePlayer = Cast<ULevelSequencePlayer>(OptionalPlayer->AsUObject());
		}

		ULevelSequenceDirector* NewDirector = NewObject<ULevelSequenceDirector>(DirectorOuter, DirectorClass, DirectorName, RF_Transient);
		NewDirector->SubSequenceID = SequenceID.GetInternalValue();
		NewDirector->WeakLinker = SharedPlaybackState->GetLinker();
		NewDirector->InstanceID = SharedPlaybackState->GetRootInstanceHandle().InstanceID;
		NewDirector->InstanceSerial = SharedPlaybackState->GetRootInstanceHandle().InstanceSerial;
		NewDirector->Player = LevelSequencePlayer;
		NewDirector->MovieScenePlayerIndex = OptionalPlayer ? OptionalPlayer->GetUniqueIndex() : INDEX_NONE;
		NewDirector->OnCreated();
		return NewDirector;
	}

	return nullptr;
}

void ULevelSequence::AddAssetUserData(UAssetUserData* InUserData)
{
	if (InUserData != NULL)
	{
		UAssetUserData* ExistingData = GetAssetUserDataOfClass(InUserData->GetClass());
		if (ExistingData != NULL)
		{
			AssetUserData.Remove(ExistingData);
		}
		AssetUserData.Add(InUserData);
	}
}

UAssetUserData* ULevelSequence::GetAssetUserDataOfClass(TSubclassOf<UAssetUserData> InUserDataClass)
{
	for (int32 DataIdx = 0; DataIdx < AssetUserData.Num(); DataIdx++)
	{
		UAssetUserData* Datum = AssetUserData[DataIdx];
		if (Datum != NULL && Datum->IsA(InUserDataClass))
		{
			return Datum;
		}
	}
	return NULL;
}

void ULevelSequence::RemoveUserDataOfClass(TSubclassOf<UAssetUserData> InUserDataClass)
{
	for (int32 DataIdx = 0; DataIdx < AssetUserData.Num(); DataIdx++)
	{
		UAssetUserData* Datum = AssetUserData[DataIdx];
		if (Datum != NULL && Datum->IsA(InUserDataClass))
		{
			AssetUserData.RemoveAt(DataIdx);
			return;
		}
	}
}

const TArray<UAssetUserData*>* ULevelSequence::GetAssetUserDataArray() const
{
	return &ToRawPtrTArrayUnsafe(AssetUserData);
}

#if WITH_EDITOR
void ULevelSequence::IterateDynamicBindings(TFunction<void(const FGuid&, FMovieSceneDynamicBinding&)> InCallback)
{
	for (FMovieSceneBindingReference& BindingReference : BindingReferences.GetAllReferences())
	{
		if (BindingReference.CustomBinding)
		{
			if (UMovieSceneReplaceableDirectorBlueprintBinding* ReplaceableDirectorBlueprintBinding = Cast<UMovieSceneReplaceableDirectorBlueprintBinding>(BindingReference.CustomBinding))
			{
				InCallback(BindingReference.ID, ReplaceableDirectorBlueprintBinding->DynamicBinding);
			}

			// We can't use 'AsSpawnable' here because we don't have playback state and we might not have a world context. 
			// This should only be called from an editor context though, so we can just check the inner spawnable.

			// If the binding is itself a spawnable director blueprint binding, then iterate over it
			if (UMovieSceneSpawnableDirectorBlueprintBinding* SpawnableDirectorBlueprintBinding = Cast<UMovieSceneSpawnableDirectorBlueprintBinding>(BindingReference.CustomBinding))
			{
				InCallback(BindingReference.ID, SpawnableDirectorBlueprintBinding->DynamicBinding);
			}
			else if (UMovieSceneReplaceableBindingBase* ReplaceableBinding = Cast<UMovieSceneReplaceableBindingBase>(BindingReference.CustomBinding))
			{
				if (UMovieSceneSpawnableDirectorBlueprintBinding* InnerSpawnableDirectorBlueprintBinding = Cast<UMovieSceneSpawnableDirectorBlueprintBinding>(ReplaceableBinding->PreviewSpawnable))
				{
					InCallback(BindingReference.ID, InnerSpawnableDirectorBlueprintBinding->DynamicBinding);
				}
			}
		}
	}
}

#endif
