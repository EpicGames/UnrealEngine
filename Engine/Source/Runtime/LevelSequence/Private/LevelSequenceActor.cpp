// Copyright Epic Games, Inc. All Rights Reserved.

#include "LevelSequenceActor.h"
#include "UObject/ConstructorHelpers.h"
#include "Engine/Texture2D.h"
#include "Engine/Level.h"
#include "Components/BillboardComponent.h"
#include "LevelSequenceBurnIn.h"
#include "DefaultLevelSequenceInstanceData.h"
#include "Engine/ActorChannel.h"
#include "Engine/LevelStreaming.h"
#include "Logging/MessageLog.h"
#include "Misc/UObjectToken.h"
#include "Net/UnrealNetwork.h"
#include "LevelSequenceModule.h"
#include "UniversalObjectLocators/ActorLocatorFragment.h"
#include "WorldPartition/WorldPartitionLevelHelper.h"
#include "MovieSceneSequenceTickManager.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LevelSequenceActor)

#if WITH_EDITOR
	#include "PropertyCustomizationHelpers.h"
	#include "ActorPickerMode.h"
	#include "SceneOutlinerFilters.h"
#endif

namespace LevelSequenceActorCVars
{
	static bool bInvalidBindingTagWarnings = true;
	static FAutoConsoleVariableRef CVarInvalidBindingTagWarnings(
		TEXT("LevelSequence.InvalidBindingTagWarnings"),
		bInvalidBindingTagWarnings,
		TEXT("Whether to emit a warning when invalid object binding tags are used to override bindings or not.\n"),
		ECVF_Default);

	static bool bMarkSequencePlayerAsGarbageOnDestroy = true;
	static FAutoConsoleVariableRef CVarMarkSequencePlayerAsGarbageOnDestroy(
		TEXT("LevelSequence.MarkSequencePlayerAsGarbageOnDestroy"),
		bMarkSequencePlayerAsGarbageOnDestroy,
		TEXT("Whether to flag the sequence player object as garbage when the actor is being destroyed"),
		ECVF_Default);
}

ALevelSequenceActor::ALevelSequenceActor(const FObjectInitializer& Init)
	: Super(Init)
	, bShowBurnin(true)
{
	USceneComponent* SceneComponent = CreateDefaultSubobject<USceneComponent>(TEXT("SceneComp"));
	RootComponent = SceneComponent;

#if WITH_EDITORONLY_DATA
	UBillboardComponent* SpriteComponent = CreateEditorOnlyDefaultSubobject<UBillboardComponent>(TEXT("Sprite"));

	if (!IsRunningCommandlet())
	{
		// Structure to hold one-time initialization
		struct FConstructorStatics
		{
			ConstructorHelpers::FObjectFinderOptional<UTexture2D> DecalTexture;
			FConstructorStatics() : DecalTexture(TEXT("/Engine/EditorResources/S_LevelSequence")) {}
		};
		static FConstructorStatics ConstructorStatics;

		if (SpriteComponent)
		{
			SpriteComponent->Sprite = ConstructorStatics.DecalTexture.Get();
			SpriteComponent->SetupAttachment(RootComponent);
			SpriteComponent->bIsScreenSizeScaled = true;
			SpriteComponent->SetUsingAbsoluteScale(true);
			SpriteComponent->bReceivesDecals = false;
			SpriteComponent->bHiddenInGame = true;
		}
	}

	bIsSpatiallyLoaded = false;
#endif //WITH_EDITORONLY_DATA

	BindingOverrides = Init.CreateDefaultSubobject<UMovieSceneBindingOverrides>(this, "BindingOverrides");
	BurnInOptions = Init.CreateDefaultSubobject<ULevelSequenceBurnInOptions>(this, "BurnInOptions");
	DefaultInstanceData = Init.CreateDefaultSubobject<UDefaultLevelSequenceInstanceData>(this, "InstanceData");

#if WITH_EDITOR
	if (UDefaultLevelSequenceInstanceData* LevelSequenceInstanceData = Cast<UDefaultLevelSequenceInstanceData>(DefaultInstanceData))
	{
		LevelSequenceInstanceData->TransformOriginChangedEvent.AddUObject(this, &ALevelSequenceActor::TransformOriginChanged);
	}
#endif // WITH_EDITOR

	// SequencePlayer must be a default sub object for it to be replicated correctly
PRAGMA_DISABLE_DEPRECATION_WARNINGS // make SequencePlayer protected and remove this for 5.6
	SequencePlayer = Init.CreateDefaultSubobject<ULevelSequencePlayer>(this, "AnimationPlayer");
PRAGMA_ENABLE_DEPRECATION_WARNINGS // SequencePlayer
	bOverrideInstanceData = false;

	// The level sequence actor defaults to never ticking by the tick manager because it is ticked separately in LevelTick
	//PrimaryActorTick.bCanEverTick = false;

	bAutoPlay_DEPRECATED = false;

	bReplicates = true;
	bReplicatePlayback = false;
	bReplicateUsingRegisteredSubObjectList = true;
}

void ALevelSequenceActor::PostInitProperties()
{
	Super::PostInitProperties();

	// Have to initialize this here as any properties set on default subobjects inside the constructor
	// Get stomped by the CDO's properties when the constructor exits.
	GetSequencePlayer()->SetPlaybackClient(this);
	GetSequencePlayer()->SetPlaybackSettings(PlaybackSettings);
}

void ALevelSequenceActor::RewindForReplay()
{
	if (GetSequencePlayer())
	{
		GetSequencePlayer()->RewindForReplay();
	}
}

bool ALevelSequenceActor::RetrieveBindingOverrides(const FGuid& InBindingId, FMovieSceneSequenceID InSequenceID, TArray<UObject*, TInlineAllocator<1>>& OutObjects) const
{
	return BindingOverrides->LocateBoundObjects(InBindingId, InSequenceID, OutObjects);
}

UObject* ALevelSequenceActor::GetInstanceData() const
{
	return bOverrideInstanceData ? DefaultInstanceData : nullptr;
}

TOptional<EAspectRatioAxisConstraint> ALevelSequenceActor::GetAspectRatioAxisConstraint() const
{
	TOptional<EAspectRatioAxisConstraint> AspectRatioAxisConstraint;
	if (CameraSettings.bOverrideAspectRatioAxisConstraint)
	{
		AspectRatioAxisConstraint = CameraSettings.AspectRatioAxisConstraint;
	}
	return AspectRatioAxisConstraint;
}

bool ALevelSequenceActor::GetIsReplicatedPlayback() const
{
	return bReplicatePlayback;
}

#if WITH_EDITOR
bool ALevelSequenceActor::CanPreview(const UMovieSceneSequence& InSequence) const
{
	return GetSequence() == &InSequence;
}
#endif

ULevelSequencePlayer* ALevelSequenceActor::GetSequencePlayer() const
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS // make SequencePlayer protected and remove this for 5.6
	return SequencePlayer;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

void ALevelSequenceActor::SetReplicatePlayback(bool bInReplicatePlayback)
{
	bReplicatePlayback = bInReplicatePlayback;
	SetReplicates(bReplicatePlayback);
}

void ALevelSequenceActor::GetLifetimeReplicatedProps(TArray<FLifetimeProperty>& OutLifetimeProps) const
{
	Super::GetLifetimeReplicatedProps(OutLifetimeProps);

PRAGMA_DISABLE_DEPRECATION_WARNINGS // make SequencePlayer protected and remove this for 5.6
	DOREPLIFETIME(ALevelSequenceActor, SequencePlayer);
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	DOREPLIFETIME(ALevelSequenceActor, LevelSequenceAsset);
}

void ALevelSequenceActor::PreInitializeComponents()
{
	Super::PreInitializeComponents();

	UWorld* StreamingWorld = nullptr;
	FTopLevelAssetPath StreamedLevelAssetPath;

	// Initialize the level streaming asset path for this actor if possible/necessary
	if (ULevel* Level = GetLevel())
	{
		// Default to owning world (to resolve AlwaysLoaded actors not part of a Streaming Level and Disabled Streaming World Partitions)
		StreamingWorld = Level->OwningWorld;

		// Construct the path to the level asset that the streamed level relates to
		ULevelStreaming* LevelStreaming = ULevelStreaming::FindStreamingLevel(Level);
		if (LevelStreaming)
		{
			StreamingWorld = Level->GetTypedOuter<UWorld>();
		}

		if (LevelStreaming)
		{
			// StreamedLevelPackage is a package name of the form /Game/Folder/MapName, not a full asset path
			FString StreamedLevelPackage = ((LevelStreaming->PackageNameToLoad == NAME_None) ? LevelStreaming->GetWorldAssetPackageFName() : LevelStreaming->PackageNameToLoad).ToString();

			int32 SlashPos = 0;
			if (StreamedLevelPackage.FindLastChar('/', SlashPos) && SlashPos < StreamedLevelPackage.Len() - 1)
			{
				StreamedLevelAssetPath = FTopLevelAssetPath(*StreamedLevelPackage, &StreamedLevelPackage[SlashPos + 1]);
			}
		}
	}

	GetSequencePlayer()->SetSourceActorContext(
		StreamingWorld,
		WorldPartitionResolveData.ContainerID,
		WorldPartitionResolveData.SourceWorldAssetPath.IsValid() ? WorldPartitionResolveData.SourceWorldAssetPath : StreamedLevelAssetPath
	);
}

void ALevelSequenceActor::PostInitializeComponents()
{
	Super::PostInitializeComponents();

	if (HasAuthority())
	{
		SetReplicates(bReplicatePlayback);
	}
	
	// Initialize this player for tick as soon as possible to ensure that a persistent
	// reference to the tick manager is maintained
	GetSequencePlayer()->InitializeForTick(this);
	GetSequencePlayer()->SetPlaybackSettings(PlaybackSettings);

	InitializePlayer();
}

void ALevelSequenceActor::BeginPlay()
{
	Super::BeginPlay();

	if (GetSequencePlayer())
	{
		AddReplicatedSubObject(GetSequencePlayer());
		GetSequencePlayer()->OnPlay.AddUniqueDynamic(this, &ALevelSequenceActor::ShowBurnin);
		GetSequencePlayer()->OnPlayReverse.AddUniqueDynamic(this, &ALevelSequenceActor::ShowBurnin);
		GetSequencePlayer()->OnStop.AddUniqueDynamic(this, &ALevelSequenceActor::HideBurnin);
	}

	if (PlaybackSettings.bAutoPlay)
	{
		GetSequencePlayer()->Play();
	}
}

void ALevelSequenceActor::EndPlay(const EEndPlayReason::Type EndPlayReason)
{
	if (ULevelSequencePlayer* Player = GetSequencePlayer())
	{
		RemoveReplicatedSubObject(Player);

		// Stop may modify a lot of actor state so it needs to be called
		// during EndPlay (when Actors + World are still valid) instead
		// of waiting for the UObject to be destroyed by GC.
		Player->Stop();

		Player->OnPlay.RemoveAll(this);
		Player->OnPlayReverse.RemoveAll(this);
		Player->OnStop.RemoveAll(this);

		Player->TearDown();

		// This actor may be being destroyed due to leaving net-relevancy, in which case we need to explicitly
		// mark the sub-object as garbage. Otherwise, re-entering relevancy will recreate the actor on the client
		// but may find and assign the existing yet-un-GC'd player sub-object from the previous actor instance.
		// Actor sub-objects may be automatically marked garbage some day, but for now we take care of it manually.
		if (LevelSequenceActorCVars::bMarkSequencePlayerAsGarbageOnDestroy && (EndPlayReason == EEndPlayReason::Destroyed))
		{
			Player->MarkAsGarbage();
		}
	}

	Super::EndPlay(EndPlayReason);
}

void ALevelSequenceActor::PostLoad()
{
	Super::PostLoad();

	// If autoplay was previously enabled, initialize the playback settings to autoplay
	if (bAutoPlay_DEPRECATED)
	{
		PlaybackSettings.bAutoPlay = bAutoPlay_DEPRECATED;
		bAutoPlay_DEPRECATED = false;
	}

	GetSequencePlayer()->SetPlaybackSettings(PlaybackSettings);

#if WITH_EDITORONLY_DATA
	if (LevelSequence_DEPRECATED.IsValid())
	{
		if (!LevelSequenceAsset)
		{
			LevelSequenceAsset = Cast<ULevelSequence>(LevelSequence_DEPRECATED.ResolveObject());
		}

		// If we don't have the sequence asset loaded, schedule a load for it
		if (LevelSequenceAsset)
		{
			LevelSequence_DEPRECATED.Reset();
		}
		else
		{
			// ~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~
			// We intentionally do not attempt to load any asset in PostLoad other than by way of LoadPackageAsync
			// since under some circumstances it is possible for the sequence to only be partially loaded.
			// ~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~
			LoadPackageAsync(LevelSequence_DEPRECATED.GetLongPackageName(), FLoadPackageAsyncDelegate::CreateUObject(this, &ALevelSequenceActor::OnSequenceLoaded));
		}
	}

	// Fix sprite component so that it's attached to the root component. In the past, the sprite component was the root component.
	UBillboardComponent* SpriteComponent = FindComponentByClass<UBillboardComponent>();
	if (SpriteComponent && SpriteComponent->GetAttachParent() != RootComponent)
	{
		SpriteComponent->SetupAttachment(RootComponent);
	}
#endif
}

#if WITH_EDITOR

void ALevelSequenceActor::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	FName PropertyName = (PropertyChangedEvent.Property != nullptr) ? PropertyChangedEvent.Property->GetFName() : NAME_None;

	if (PropertyName == GET_MEMBER_NAME_CHECKED(ALevelSequenceActor, bOverrideInstanceData))
	{
		TransformOriginChanged();
	}

	Super::PostEditChangeProperty(PropertyChangedEvent);
}

void ALevelSequenceActor::TransformOriginChanged()
{
	if (LevelSequenceAsset)
	{
		if (UMovieScene* MovieScene = LevelSequenceAsset->GetMovieScene())
		{
			MovieScene->MarkAsChanged();
		}
	}
}

#endif // #if WITH_EDITOR


#if WITH_EDITORONLY_DATA
void ALevelSequenceActor::DeclareConstructClasses(TArray<FTopLevelAssetPath>& OutConstructClasses, const UClass* SpecificSubclass)
{
	Super::DeclareConstructClasses(OutConstructClasses, SpecificSubclass);
	OutConstructClasses.Add(FTopLevelAssetPath(UDefaultLevelSequenceInstanceData::StaticClass()));
}
#endif

ULevelSequence* ALevelSequenceActor::GetSequence() const
{
	return LevelSequenceAsset;
}

void ALevelSequenceActor::SetSequence(ULevelSequence* InSequence)
{
	if (!GetSequencePlayer()->IsPlaying())
	{
		LevelSequenceAsset = InSequence;

		// cbb: should ideally null out the template and player when no sequence is assigned, but that's currently not possible
		if (InSequence)
		{
			GetSequencePlayer()->Initialize(InSequence, GetLevel(), CameraSettings);
		}
	}
}

void ALevelSequenceActor::InitializePlayer()
{
	if (LevelSequenceAsset && GetWorld()->IsGameWorld())
	{
		// Level sequence is already loaded. Initialize the player if it's not already initialized with this sequence
		if (LevelSequenceAsset != GetSequencePlayer()->GetSequence() || GetSequencePlayer()->GetEvaluationTemplate().GetRunner() == nullptr)
		{
			GetSequencePlayer()->Initialize(LevelSequenceAsset, GetLevel(), CameraSettings);
		}
	}
}

void ALevelSequenceActor::OnSequenceLoaded(const FName& PackageName, UPackage* Package, EAsyncLoadingResult::Type Result)
{
	if (Result == EAsyncLoadingResult::Succeeded)
	{
#if WITH_EDITORONLY_DATA
		if (LevelSequence_DEPRECATED.IsValid())
		{
			LevelSequenceAsset = Cast<ULevelSequence>(LevelSequence_DEPRECATED.ResolveObject());
			LevelSequence_DEPRECATED.Reset();
		}
#endif
	}
}

void ALevelSequenceActor::HideBurnin()
{
	bShowBurnin = false;
	RefreshBurnIn();
}

void ALevelSequenceActor::ShowBurnin()
{
	bShowBurnin = true;
	RefreshBurnIn();
}

void ALevelSequenceActor::RefreshBurnIn()
{
	if (BurnInInstance)
	{
		BurnInInstance->RemoveFromParent();
		BurnInInstance = nullptr;
	}
	
	if (BurnInOptions && BurnInOptions->bUseBurnIn && bShowBurnin)
	{
		// Create the burn-in if necessary
		UClass* Class = BurnInOptions->BurnInClass.TryLoadClass<ULevelSequenceBurnIn>();
		if (Class)
		{
			BurnInInstance = CreateWidget<ULevelSequenceBurnIn>(GetWorld(), Class);
			if (BurnInInstance)
			{
				// Ensure we have a valid settings object if possible
				BurnInOptions->ResetSettings();

				BurnInInstance->SetSettings(BurnInOptions->Settings);
				BurnInInstance->TakeSnapshotsFrom(*this);
				BurnInInstance->AddToViewport();
			}
		}
	}
}

void ALevelSequenceActor::SetBinding(FMovieSceneObjectBindingID Binding, const TArray<AActor*>& Actors, bool bAllowBindingsFromAsset)
{
	if (!Binding.IsValid())
	{
		FMessageLog("PIE")
			.Warning(NSLOCTEXT("LevelSequenceActor", "SetBinding_Warning", "The specified binding ID is not valid"))
			->AddToken(FUObjectToken::Create(this));
	}
	else
	{
		BindingOverrides->SetBinding(Binding, TArray<UObject*>(Actors), bAllowBindingsFromAsset);
		if (GetSequencePlayer())
		{
			FMovieSceneSequenceID SequenceID = Binding.ResolveSequenceID(MovieSceneSequenceID::Root, *GetSequencePlayer());
			GetSequencePlayer()->GetEvaluationState()->Invalidate(Binding.GetGuid(), SequenceID);
		}
	}
}

void ALevelSequenceActor::SetBindingByTag(FName BindingTag, const TArray<AActor*>& Actors, bool bAllowBindingsFromAsset)
{
	const UMovieSceneSequence*         Sequence = GetSequence();
	const FMovieSceneObjectBindingIDs* Bindings = Sequence ? Sequence->GetMovieScene()->AllTaggedBindings().Find(BindingTag) : nullptr;
	if (Bindings)
	{
		for (FMovieSceneObjectBindingID BindingID : Bindings->IDs)
		{
			SetBinding(BindingID, Actors, bAllowBindingsFromAsset);
		}
	}
	else if (LevelSequenceActorCVars::bInvalidBindingTagWarnings)
	{
		FMessageLog("PIE")
			.Warning(FText::Format(NSLOCTEXT("LevelSequenceActor", "SetBindingByTag", "Sequence did not contain any bindings with the tag '{0}'"), FText::FromName(BindingTag)))
			->AddToken(FUObjectToken::Create(this));
	}
}

void ALevelSequenceActor::AddBinding(FMovieSceneObjectBindingID Binding, AActor* Actor, bool bAllowBindingsFromAsset)
{
	if (!Binding.IsValid())
	{
		FMessageLog("PIE")
			.Warning(NSLOCTEXT("LevelSequenceActor", "AddBinding_Warning", "The specified binding ID is not valid"))
			->AddToken(FUObjectToken::Create(this));
	}
	else
	{
		BindingOverrides->AddBinding(Binding, Actor, bAllowBindingsFromAsset);
		if (GetSequencePlayer())
		{
			FMovieSceneSequenceID SequenceID = Binding.ResolveSequenceID(MovieSceneSequenceID::Root, *GetSequencePlayer());
			GetSequencePlayer()->GetEvaluationState()->Invalidate(Binding.GetGuid(), SequenceID);
		}
	}
}

void ALevelSequenceActor::AddBindingByTag(FName BindingTag, AActor* Actor, bool bAllowBindingsFromAsset)
{
	const UMovieSceneSequence*         Sequence = GetSequence();
	const FMovieSceneObjectBindingIDs* Bindings = Sequence ? Sequence->GetMovieScene()->AllTaggedBindings().Find(BindingTag) : nullptr;
	if (Bindings)
	{
		for (FMovieSceneObjectBindingID BindingID : Bindings->IDs)
		{
			AddBinding(BindingID, Actor, bAllowBindingsFromAsset);
		}
	}
	else if (LevelSequenceActorCVars::bInvalidBindingTagWarnings)
	{
		FMessageLog("PIE")
			.Warning(FText::Format(NSLOCTEXT("LevelSequenceActor", "AddBindingByTag", "Sequence did not contain any bindings with the tag '{0}'"), FText::FromName(BindingTag)))
			->AddToken(FUObjectToken::Create(this));
	}
}

void ALevelSequenceActor::RemoveBinding(FMovieSceneObjectBindingID Binding, AActor* Actor)
{
	if (!Binding.IsValid())
	{
		FMessageLog("PIE")
			.Warning(NSLOCTEXT("LevelSequenceActor", "RemoveBinding_Warning", "The specified binding ID is not valid"))
			->AddToken(FUObjectToken::Create(this));
	}
	else
	{
		BindingOverrides->RemoveBinding(Binding, Actor);
		if (GetSequencePlayer())
		{
			FMovieSceneSequenceID SequenceID = Binding.ResolveSequenceID(MovieSceneSequenceID::Root, *GetSequencePlayer());
			GetSequencePlayer()->GetEvaluationState()->Invalidate(Binding.GetGuid(), SequenceID);
		}
	}
}

void ALevelSequenceActor::RemoveBindingByTag(FName BindingTag, AActor* Actor)
{
	const UMovieSceneSequence*         Sequence = GetSequence();
	const FMovieSceneObjectBindingIDs* Bindings = Sequence ? Sequence->GetMovieScene()->AllTaggedBindings().Find(BindingTag) : nullptr;
	if (Bindings)
	{
		for (FMovieSceneObjectBindingID BindingID : Bindings->IDs)
		{
			RemoveBinding(BindingID, Actor);
		}
	}
	else if (LevelSequenceActorCVars::bInvalidBindingTagWarnings)
	{
		FMessageLog("PIE")
			.Warning(FText::Format(NSLOCTEXT("LevelSequenceActor", "RemoveBindingByTag", "Sequence did not contain any bindings with the tag '{0}'"), FText::FromName(BindingTag)))
			->AddToken(FUObjectToken::Create(this));
	}
}

void ALevelSequenceActor::ResetBinding(FMovieSceneObjectBindingID Binding)
{
	if (!Binding.IsValid())
	{
		FMessageLog("PIE")
			.Warning(NSLOCTEXT("LevelSequenceActor", "ResetBinding_Warning", "The specified binding ID is not valid"))
			->AddToken(FUObjectToken::Create(this));
	}
	else
	{
		BindingOverrides->ResetBinding(Binding);
		if (GetSequencePlayer())
		{
			FMovieSceneSequenceID SequenceID = Binding.ResolveSequenceID(MovieSceneSequenceID::Root, *GetSequencePlayer());
			GetSequencePlayer()->GetEvaluationState()->Invalidate(Binding.GetGuid(), SequenceID);
		}
	}
}

void ALevelSequenceActor::ResetBindings()
{
	BindingOverrides->ResetBindings();
	if (GetSequencePlayer())
	{
		GetSequencePlayer()->GetEvaluationState()->ClearObjectCaches(*GetSequencePlayer());
	}
}

FMovieSceneObjectBindingID ALevelSequenceActor::FindNamedBinding(FName InBindingName) const
{
	if (ensureAlways(GetSequencePlayer()))
	{
		return GetSequencePlayer()->GetSequence()->FindBindingByTag(InBindingName);
	}
	return FMovieSceneObjectBindingID();
}

const TArray<FMovieSceneObjectBindingID>& ALevelSequenceActor::FindNamedBindings(FName InBindingName) const
{
	if (ensureAlways(GetSequencePlayer()))
	{
		return GetSequencePlayer()->GetSequence()->FindBindingsByTag(InBindingName);
	}

	static TArray<FMovieSceneObjectBindingID> EmptyBindings;
	return EmptyBindings;
}

void ALevelSequenceActor::PostNetReceive()
{
	Super::PostNetReceive();

	InitializePlayer();
}

#if WITH_EDITOR

void FBoundActorProxy::Initialize(TSharedPtr<IPropertyHandle> InPropertyHandle)
{
	ReflectedProperty = InPropertyHandle;

	UObject* Object = nullptr;
	ReflectedProperty->GetValue(Object);
	BoundActor = Cast<AActor>(Object);

	ReflectedProperty->SetOnPropertyValueChanged(FSimpleDelegate::CreateRaw(this, &FBoundActorProxy::OnReflectedPropertyChanged));
}

void FBoundActorProxy::OnReflectedPropertyChanged()
{
	UObject* Object = nullptr;
	ReflectedProperty->GetValue(Object);
	BoundActor = Cast<AActor>(Object);
}

TSharedPtr<FStructOnScope> ALevelSequenceActor::GetObjectPickerProxy(TSharedPtr<IPropertyHandle> ObjectPropertyHandle)
{
	TSharedRef<FStructOnScope> Struct = MakeShared<FStructOnScope>(FBoundActorProxy::StaticStruct());
	reinterpret_cast<FBoundActorProxy*>(Struct->GetStructMemory())->Initialize(ObjectPropertyHandle);
	return Struct;
}

void ALevelSequenceActor::UpdateObjectFromProxy(FStructOnScope& Proxy, IPropertyHandle& ObjectPropertyHandle)
{
	UObject* BoundActor = reinterpret_cast<FBoundActorProxy*>(Proxy.GetStructMemory())->BoundActor;
	ObjectPropertyHandle.SetValue(BoundActor);
}

UMovieSceneSequence* ALevelSequenceActor::RetrieveOwnedSequence() const
{
	return GetSequence();
}

bool ALevelSequenceActor::GetReferencedContentObjects(TArray<UObject*>& Objects) const
{
	if (LevelSequenceAsset)
	{
		Objects.Add(LevelSequenceAsset);
	}

	Super::GetReferencedContentObjects(Objects);

	return true;
}

#endif



ULevelSequenceBurnInOptions::ULevelSequenceBurnInOptions(const FObjectInitializer& Init)
	: Super(Init)
	, bUseBurnIn(false)
	, BurnInClass(TEXT("/Engine/Sequencer/DefaultBurnIn.DefaultBurnIn_C"))
	, Settings(nullptr)
{
}

void ULevelSequenceBurnInOptions::SetBurnIn(FSoftClassPath InBurnInClass)
{
	BurnInClass = InBurnInClass;
	
	// Attempt to load the settings class from the BurnIn class and assign it to our local Settings object.
	ResetSettings();
}


void ULevelSequenceBurnInOptions::ResetSettings()
{
	UClass* Class = BurnInClass.TryLoadClass<ULevelSequenceBurnIn>();
	if (Class)
	{
		TSubclassOf<ULevelSequenceBurnInInitSettings> SettingsClass = Cast<ULevelSequenceBurnIn>(Class->GetDefaultObject())->GetSettingsClass();
		if (SettingsClass)
		{
			if (!Settings || !Settings->IsA(SettingsClass))
			{
				if (Settings)
				{
					Settings->Rename(*MakeUniqueObjectName(this, ULevelSequenceBurnInInitSettings::StaticClass(), "Settings_EXPIRED").ToString());
				}
				
				Settings = NewObject<ULevelSequenceBurnInInitSettings>(this, SettingsClass, "Settings");
				Settings->SetFlags(GetMaskedFlags(RF_PropagateToSubObjects));
			}
		}
		else
		{
			Settings = nullptr;
		}
	}
	else
	{
		Settings = nullptr;
	}
}

#if WITH_EDITOR

void ULevelSequenceBurnInOptions::PostEditChangeProperty( FPropertyChangedEvent& PropertyChangedEvent)
{
	FName PropertyName = (PropertyChangedEvent.Property != nullptr) ? PropertyChangedEvent.Property->GetFName() : NAME_None;

	if (PropertyName == GET_MEMBER_NAME_CHECKED(ULevelSequenceBurnInOptions, bUseBurnIn) || PropertyName == GET_MEMBER_NAME_CHECKED(ULevelSequenceBurnInOptions, BurnInClass))
	{
		ResetSettings();
	}

	Super::PostEditChangeProperty(PropertyChangedEvent);
}

#endif // WITH_EDITOR

AReplicatedLevelSequenceActor::AReplicatedLevelSequenceActor(const FObjectInitializer& Init)
	: Super(Init)
{
	bAlwaysRelevant = true;
}
