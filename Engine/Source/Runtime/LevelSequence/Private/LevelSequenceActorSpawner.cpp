// Copyright Epic Games, Inc. All Rights Reserved.

#include "LevelSequenceActorSpawner.h"
#include "Misc/PackageName.h"
#include "MovieSceneSpawnable.h"
#include "GameFramework/Actor.h"
#include "Engine/World.h"
#include "Particles/ParticleSystemComponent.h"
#include "Engine/Engine.h"
#include "Engine/Level.h"
#include "Engine/LevelStreaming.h"
#include "LevelUtils.h"
#include "UObject/Package.h"
#include "UObject/UObjectGlobals.h"

#include "Evaluation/MovieSceneEvaluationTemplateInstance.h"
#include "EntitySystem/MovieSceneEntitySystemLinker.h"
#include "Systems/MovieSceneDeferredComponentMovementSystem.h"

#if WITH_EDITOR
#include "Editor.h"
#endif

static const FName SequencerActorTag("SequencerActor");

TSharedRef<IMovieSceneObjectSpawner> FLevelSequenceActorSpawner::CreateObjectSpawner()
{
	return MakeShareable(new FLevelSequenceActorSpawner);
}

UClass* FLevelSequenceActorSpawner::GetSupportedTemplateType() const
{
	return AActor::StaticClass();
}

ULevelStreaming* GetLevelStreaming(const FName& DesiredLevelName, const UWorld* World)
{
	if (DesiredLevelName == NAME_None)
	{
		return nullptr;
	}

	const TArray<ULevelStreaming*>& StreamingLevels = World->GetStreamingLevels();
	FString SafeLevelNameString = DesiredLevelName.ToString();
	if (FPackageName::IsShortPackageName(SafeLevelNameString))
	{
		// Make sure MyMap1 and Map1 names do not resolve to a same streaming level
		SafeLevelNameString.InsertAt(0, '/');
	}

#if WITH_EDITOR
	FWorldContext* WorldContext = GEngine->GetWorldContextFromWorld(World);
	if (WorldContext && WorldContext->PIEInstance != INDEX_NONE)
	{
		SafeLevelNameString = UWorld::ConvertToPIEPackageName(SafeLevelNameString, WorldContext->PIEInstance);
	}
#endif


	for (ULevelStreaming* LevelStreaming : StreamingLevels)
	{
		if (LevelStreaming && LevelStreaming->GetWorldAssetPackageName().EndsWith(SafeLevelNameString, ESearchCase::IgnoreCase))
		{
			return LevelStreaming;
		}
	}

	return nullptr;
}

UObject* FLevelSequenceActorSpawner::SpawnObject(FMovieSceneSpawnable& Spawnable, FMovieSceneSequenceIDRef TemplateID, TSharedRef<const UE::MovieScene::FSharedPlaybackState> SharedPlaybackState)
{
	AActor* ObjectTemplate = Cast<AActor>(Spawnable.GetObjectTemplate());
	if (!ObjectTemplate)
	{
		return nullptr;
	}

	if (!ensure(!ObjectTemplate->GetClass()->HasAnyClassFlags(CLASS_NewerVersionExists)))
	{
		return nullptr;
	}

	const EObjectFlags ObjectFlags = RF_Transient | RF_Transactional;

	// @todo sequencer livecapture: Consider using SetPlayInEditorWorld() and RestoreEditorWorld() here instead
	
	// @todo sequencer actors: We need to make sure puppet objects aren't copied into PIE/SIE sessions!  They should be omitted from that duplication!

	UObject* PlaybackContext = SharedPlaybackState->GetPlaybackContext();
	UWorld* WorldContext = PlaybackContext ? PlaybackContext->GetWorld() : nullptr;

	FName DesiredLevelName = Spawnable.GetLevelName();
	if (DesiredLevelName != NAME_None)
	{
		if (WorldContext && WorldContext->GetFName() == DesiredLevelName)
		{
			// done, spawn into this world
		}
		else
		{
			ULevelStreaming* LevelStreaming = GetLevelStreaming(DesiredLevelName, WorldContext);
			if (LevelStreaming && LevelStreaming->GetWorldAsset().IsValid())
			{
				WorldContext = LevelStreaming->GetWorldAsset().Get();
			}
			else
			{
				UE_LOGF(LogMovieScene, Warning, "Can't find sublevel '%ls' to spawn '%ls' into, defaulting to Persistent level", *DesiredLevelName.ToString(), *Spawnable.GetName());
			}
		}
	}

	if (WorldContext == nullptr)
	{
		UE_LOGF(LogMovieScene, Warning, "Can't find world to spawn '%ls' into, defaulting to Persistent level", *Spawnable.GetName());

		WorldContext = GWorld;
	}

	// We use the net addressable name for spawnables on any non-editor, non-standalone world (ie, all clients, servers and PIE worlds)
	const bool bUseNetAddressableName = Spawnable.bNetAddressableName && (WorldContext->WorldType != EWorldType::Editor) && (WorldContext->GetNetMode() != ENetMode::NM_Standalone);

	FName SpawnName = bUseNetAddressableName ? Spawnable.GetNetAddressableName(SharedPlaybackState, TemplateID) :
#if WITH_EDITOR
		// Construct the object with the same name that we will set later on the actor to avoid renaming it inside SetActorLabel
		MakeUniqueObjectName(WorldContext->PersistentLevel, ObjectTemplate->GetClass(), *Spawnable.GetName());
#else
		NAME_None;
#endif

	// If there's an object that already exists with the requested name, it needs to be renamed (it's probably pending kill)
	if (!SpawnName.IsNone())
	{
		UObject* ExistingObject = StaticFindObjectFast(nullptr, WorldContext->PersistentLevel, SpawnName);
		if (ExistingObject)
		{
			FName DefunctName = MakeUniqueObjectName(WorldContext->PersistentLevel, ExistingObject->GetClass());
			ExistingObject->Rename(*DefunctName.ToString(), nullptr);
		}
	}

	FStringView SpawnLabel;

#if WITH_EDITOR
	const IConsoleVariable* AllowSetActorLabelCvar = IConsoleManager::Get().FindConsoleVariable(TEXT("LevelSequence.EnableReadableActorLabelsForSpawnables"));
	const bool bAllowSetActorLabel = AllowSetActorLabelCvar && AllowSetActorLabelCvar->GetBool();
	
	// Historically, setting the actor label has caused performance issues in some scenarios (by causing async loading flushes); however, there's no
	// evidence for this anymore, so the cvar is here to turn off this behavior if needed.
	if ((WorldContext->WorldType == EWorldType::Editor) || bAllowSetActorLabel)
	{
		SpawnLabel = Spawnable.GetName();
	}
#endif

	// Spawn the puppet actor
	FActorSpawnParameters SpawnInfo;
	{
		SpawnInfo.Name = SpawnName;
#if WITH_EDITOR
		SpawnInfo.InitialActorLabel = SpawnLabel;
#endif
		SpawnInfo.ObjectFlags = ObjectFlags;
		SpawnInfo.SpawnCollisionHandlingOverride = ESpawnActorCollisionHandlingMethod::AlwaysSpawn;
		// @todo: Spawning with a non-CDO template is fraught with issues
		//SpawnInfo.Template = ObjectTemplate;
		// allow pre-construction variables to be set.
		SpawnInfo.bDeferConstruction = true;
		SpawnInfo.Template = ObjectTemplate;
		SpawnInfo.OverrideLevel = WorldContext->PersistentLevel;
	}

	FTransform SpawnTransform;

	if (USceneComponent* RootComponent = ObjectTemplate->GetRootComponent())
	{
		SpawnTransform.SetTranslation(RootComponent->GetRelativeLocation());
		SpawnTransform.SetRotation(RootComponent->GetRelativeRotation().Quaternion());
		SpawnTransform.SetScale3D(RootComponent->GetRelativeScale3D());
	}
	else
	{
		SpawnTransform = Spawnable.SpawnTransform;
	}

	{
		// Disable all particle components so that they don't auto fire as soon as the actor is spawned. The particles should be triggered through the particle track.
		for (UActorComponent* Component : ObjectTemplate->GetComponents())
		{
			if (UParticleSystemComponent* ParticleComponent = Cast<UParticleSystemComponent>(Component))
			{
				// The particle needs to be set inactive in case its template was active.
				ParticleComponent->SetActiveFlag(false);
				Component->bAutoActivate = false;
			}
		}
	}

	AActor* SpawnedActor = WorldContext->SpawnActorAbsolute(ObjectTemplate->GetClass(), SpawnTransform, SpawnInfo);
	if (!SpawnedActor)
	{
		return nullptr;
	}
	
	//UEngine::FCopyPropertiesForUnrelatedObjectsParams CopyParams;
	//CopyParams.bPreserveRootComponent = false;
	//CopyParams.bNotifyObjectReplacement = false;
	//SpawnedActor->UnregisterAllComponents();
	//UEngine::CopyPropertiesForUnrelatedObjects(ObjectTemplate, SpawnedActor, CopyParams);
	//SpawnedActor->RegisterAllComponents();

	// Ensure this spawnable is not a preview actor. Preview actors will not have BeginPlay() called on them.
#if WITH_EDITOR
	SpawnedActor->bIsEditorPreviewActor = false;
#endif

	// tag this actor so we know it was spawned by sequencer
	SpawnedActor->Tags.AddUnique(SequencerActorTag);
	if (bUseNetAddressableName)
	{
		SpawnedActor->SetNetAddressable();
	}

#if WITH_EDITOR
	if (GIsEditor)
	{
		// Explicitly set RF_Transactional on spawned actors so we can undo/redo properties on them.
		SpawnedActor->SetFlags(RF_Transactional);

		for (UActorComponent* Component : SpawnedActor->GetComponents())
		{
			if (Component)
			{
				Component->SetFlags(RF_Transactional);
			}
		}
	}	
#endif

	if (UMovieSceneEntitySystemLinker* Linker = SharedPlaybackState->GetLinker())
	{
		if (UMovieSceneDeferredComponentMovementSystem* DeferredMovementSystem = Linker->FindSystem<UMovieSceneDeferredComponentMovementSystem>())
		{
			for (UActorComponent* ActorComponent : SpawnedActor->GetComponents())
			{
				if (USceneComponent* SceneComponent = Cast<USceneComponent>(ActorComponent))
				{
					DeferredMovementSystem->DeferMovementUpdates(SceneComponent);
				}
			}
		}
	}

	const bool bIsDefaultTransform = true;
	SpawnedActor->FinishSpawning(SpawnTransform, bIsDefaultTransform);

	return SpawnedActor;
}

void FLevelSequenceActorSpawner::DestroySpawnedObject(UObject& Object)
{
	AActor* Actor = Cast<AActor>(&Object);
	if (!ensure(Actor))
	{
		return;
	}

#if WITH_EDITOR
	if (GIsEditor)
	{
		// Explicitly remove RF_Transactional on spawned actors since we don't want to trasact spawn/destroy events
		Actor->ClearFlags(RF_Transactional);
		for (UActorComponent* Component : Actor->GetComponents())
		{
			if (Component)
			{
				Component->ClearFlags(RF_Transactional);
			}
		}
	}
#endif

	UWorld* World = Actor->GetWorld();
	if (World)
	{
		const bool bNetForce = false;
		const bool bShouldModifyLevel = false;
		World->DestroyActor(Actor, bNetForce, bShouldModifyLevel);
	}
}
