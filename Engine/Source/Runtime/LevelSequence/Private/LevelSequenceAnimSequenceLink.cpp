// Copyright Epic Games, Inc. All Rights Reserved.

#include "LevelSequenceAnimSequenceLink.h"
#include "Animation/AnimSequence.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LevelSequenceAnimSequenceLink)


void FLevelSequenceAnimSequenceLinkItem::SetAnimSequence(UAnimSequence* InAnimSequence)
{
	PathToAnimSequence = FSoftObjectPath(InAnimSequence);
}

UAnimSequence* FLevelSequenceAnimSequenceLinkItem::ResolveAnimSequence()
{
	UObject *Object = PathToAnimSequence.TryLoad();
	return Cast<UAnimSequence>(Object);
}


ULevelSequenceAnimSequenceLink::ULevelSequenceAnimSequenceLink(class FObjectInitializer const& ObjectInitializer)
	: Super(ObjectInitializer)
{

}
