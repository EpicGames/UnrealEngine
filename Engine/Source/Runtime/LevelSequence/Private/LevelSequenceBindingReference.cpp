// Copyright Epic Games, Inc. All Rights Reserved.

#include "LevelSequenceBindingReference.h"
#include "Modules/ModuleManager.h"
#include "Engine/Level.h"
#include "UObject/GarbageCollection.h"
#include "UObject/Package.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"
#include "UnrealEngine.h"
#include "MovieSceneFwd.h"
#include "Misc/PackageName.h"
#include "Engine/World.h"
#include "Animation/AnimInstance.h"
#include "Components/SkeletalMeshComponent.h"
#include "Engine/LevelStreamingDynamic.h"
#include "WorldPartition/IWorldPartitionObjectResolver.h"
#include "IMovieSceneBoundObjectProxy.h"
#include "IUniversalObjectLocatorModule.h"
#include "LevelSequenceLegacyObjectReference.h"
#include "UniversalObjectLocators/ActorLocatorFragment.h"
#include "UniversalObjectLocators/AnimInstanceLocatorFragment.h"
#include "SubObjectLocator.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LevelSequenceBindingReference)


FLevelSequenceBindingReference::FLevelSequenceBindingReference(UObject* InObject, UObject* InContext)
{
	check(InContext && InObject);

	if (!InContext->IsA<UWorld>() && InObject->IsIn(InContext))
	{
		ObjectPath = InObject->GetPathName(InContext);
	}
	else
	{
		FString FullPath = InObject->GetPathName();

#if WITH_EDITORONLY_DATA
		UPackage* ObjectPackage = InObject->GetOutermost();

		if (ensure(ObjectPackage))
		{
			// If this is being set from PIE we need to remove the pie prefix and point to the editor object
			if (ObjectPackage->GetPIEInstanceID() != INDEX_NONE)
			{
				FString PIEPrefix = FString::Printf(PLAYWORLD_PACKAGE_PREFIX TEXT("_%d_"), ObjectPackage->GetPIEInstanceID());
				FullPath.ReplaceInline(*PIEPrefix, TEXT(""));
			}
		}
#endif
		
		ExternalObjectPath = FSoftObjectPath(FullPath);
	}
}

UObject* FLevelSequenceBindingReference::Resolve(UObject* InContext, const FResolveBindingParams& InResolveBindingParams) const
{
	// tidy up todo: StreamedLevelAsset path and WorldPartitionResolveData code paths could probably share most their code
	//				 InContext could be reverted back to always be the UWorld in both paths and instead StreamedLevelAsset code path could use the StreamingWorld to resolve

	if (InContext && InContext->IsA<AActor>())
	{
		if (ExternalObjectPath.IsNull())
		{
			if (UE::IsSavingPackage(nullptr) || IsGarbageCollecting())
			{
				return nullptr;
			}

			return FindObject<UObject>(InContext, *ObjectPath, EFindObjectFlags::None);
		}
	}
	else if (InContext && InContext->IsA<ULevel>() && InResolveBindingParams.StreamedLevelAssetPath.IsValid() && ExternalObjectPath.GetAssetPath() == InResolveBindingParams.StreamedLevelAssetPath)
	{
		if (UE::IsSavingPackage(nullptr) || IsGarbageCollecting())
		{
			return nullptr;
		}

		UObject* ResolvedObject = nullptr;
		// ExternalObjectPath.GetSubPathString() specifies the path from the package (so includes PersistentLevel.) so we must do a ResolveSubObject from its outer
		UObject* ContextOuter = InContext->GetOuter();
		check(ContextOuter);
		ContextOuter->ResolveSubobject(*ExternalObjectPath.GetSubPathString(), ResolvedObject, /*bLoadIfExists*/false);
		return ResolvedObject;
	} 
	else if (UObject* ResolvedObject = nullptr; ExternalObjectPath.IsValid() && InResolveBindingParams.WorldPartitionResolveData && InResolveBindingParams.WorldPartitionResolveData->ResolveObject(InResolveBindingParams.StreamingWorld, ExternalObjectPath, ResolvedObject))
	{
		return ResolvedObject;
	}
	else
	{
		FSoftObjectPath TempPath = ExternalObjectPath;

		// Soft Object Paths don't follow asset redirectors when attempting to call ResolveObject or TryLoad.
		// We want to follow the asset redirector so that maps that have been renamed (from Untitled to their first asset name)
		// properly resolve. This fixes Possessable bindings losing their references the first time you save a map.
		TempPath.PreSavePath();

	#if WITH_EDITORONLY_DATA
		// Sequencer is explicit about providing a resolution context for its bindings. We never want to resolve to objects
		// with a different PIE instance ID, even if the current callstack is being executed inside a different GPlayInEditorID
		// scope. Since ResolveObject will always call FixupForPIE in editor based on GPlayInEditorID, we always override the current
		// GPlayInEditorID to be the current PIE instance of the provided context.
		const int32 ContextPlayInEditorID = InContext ? InContext->GetOutermost()->GetPIEInstanceID() : INDEX_NONE;
		FTemporaryPlayInEditorIDOverride PIEGuard(ContextPlayInEditorID);
	#endif

		return TempPath.ResolveObject();
	}

	return nullptr;
}

bool FLevelSequenceBindingReference::operator==(const FLevelSequenceBindingReference& Other) const
{
	return ExternalObjectPath == Other.ExternalObjectPath && ObjectPath == Other.ObjectPath;
}

void FLevelSequenceBindingReference::PostSerialize(const FArchive& Ar)
{
	if (Ar.IsLoading() && !PackageName_DEPRECATED.IsEmpty())
	{
		// This was saved as two strings, combine into one soft object path so it handles PIE and redirectors properly
		FString FullPath = PackageName_DEPRECATED + TEXT(".") + ObjectPath;

		ExternalObjectPath.SetPath(FullPath);
		ObjectPath.Reset();
		PackageName_DEPRECATED.Reset();
	}
}

UObject* ResolveByPath(UObject* InContext, const FString& InObjectPath)
{
	if (UE::IsSavingPackage(nullptr) || IsGarbageCollecting())
	{
		return nullptr;
	}

	if (!InObjectPath.IsEmpty())
	{
		if (UObject* FoundObject = FindObject<UObject>(InContext, *InObjectPath, EFindObjectFlags::None))
		{
			return FoundObject;
		}

#if WITH_EDITOR
		UWorld* WorldContext = InContext->GetWorld();
		if (WorldContext && WorldContext->IsPlayInEditor())
		{
			FString PackageRoot, PackagePath, PackageName;
			if (FPackageName::SplitLongPackageName(InObjectPath, PackageRoot, PackagePath, PackageName))
			{
				int32 ObjectDelimiterIdx = INDEX_NONE;
				PackageName.FindChar('.', ObjectDelimiterIdx);
				const FString SubLevelObjPath = PackageName.Mid(ObjectDelimiterIdx + 1);

				for (ULevel* Level : WorldContext->GetLevels())
				{
					UPackage* Pkg = Level->GetOutermost();
					if (UObject* FoundObject = FindObject<UObject>(Pkg, *SubLevelObjPath, EFindObjectFlags::None))
 					{
 						return FoundObject;
					}
				}
			}
		}
#endif

		if (UObject* FoundObject = FindFirstObject<UObject>(*InObjectPath, EFindFirstObjectOptions::NativeFirst))
		{
			return FoundObject;
		}
	}

	return nullptr;
}

void FUpgradedLevelSequenceBindingReferences::AddBinding(const FGuid& ObjectId, UObject* InObject, UObject* InContext)
{
	FUniversalObjectLocator NewLocator(InObject, InContext);
	if (!NewLocator.IsEmpty())
	{
		FMovieSceneBindingReferences::AddBinding(ObjectId, MoveTemp(NewLocator));
	}
}

bool FUpgradedLevelSequenceBindingReferences::SerializeFromMismatchedTag(const FPropertyTag& Tag, FStructuredArchive::FSlot Slot)
{
	using namespace UE::UniversalObjectLocator;

	if (!Tag.GetType().IsStruct(FLevelSequenceBindingReferences::StaticStruct()->GetFName()))
	{
		return false;
	}

	FLevelSequenceBindingReferences Legacy;
	FLevelSequenceBindingReferences::StaticStruct()->SerializeItem(Slot, &Legacy, nullptr);

	for (TPair<FGuid, FLevelSequenceBindingReferenceArray>& Pair : Legacy.BindingIdToReferences)
	{
		for (FLevelSequenceBindingReference& LegacyRef : Pair.Value.References)
		{
			if (LegacyRef.ExternalObjectPath.IsNull())
			{
				// Make a copy and add the object path
				FUniversalObjectLocator NewLocator;
				NewLocator.AddFragment<FSubObjectLocator>(MoveTemp(LegacyRef.ObjectPath));

				FMovieSceneBindingReferences::AddBinding(Pair.Key, MoveTemp(NewLocator));
			}
			else
			{
				FUniversalObjectLocator NewLocator;
				NewLocator.AddFragment<FActorLocatorFragment>(MoveTemp(LegacyRef.ExternalObjectPath));

				FMovieSceneBindingReferences::AddBinding(Pair.Key, MoveTemp(NewLocator));
			}
		}
	}
	for (const FGuid& AnimInstance : Legacy.AnimSequenceInstances)
	{
		FUniversalObjectLocator NewLocator;
		NewLocator.AddFragment<FAnimInstanceLocatorFragment>(EAnimInstanceLocatorFragmentType::AnimInstance);

		FMovieSceneBindingReferences::AddBinding(AnimInstance, MoveTemp(NewLocator));
	}

	for (const FGuid& AnimInstance : Legacy.PostProcessInstances)
	{
		FUniversalObjectLocator NewLocator;
		NewLocator.AddFragment<FAnimInstanceLocatorFragment>(EAnimInstanceLocatorFragmentType::PostProcessAnimInstance);

		FMovieSceneBindingReferences::AddBinding(AnimInstance, MoveTemp(NewLocator));
	}
	return true;
}

bool FLevelSequenceObjectReferenceMap::Serialize(FArchive& Ar)
{
	int32 Num = Map.Num();
	Ar << Num;

	if (Ar.IsLoading())
	{
		while(Num-- > 0)
		{
			FGuid Key;
			Ar << Key;

			FLevelSequenceLegacyObjectReference Value;
			Ar << Value;

			Map.Add(Key, Value);
		}
	}
	else if (Ar.IsSaving() || Ar.IsCountingMemory() || Ar.IsObjectReferenceCollector())
	{
		for (auto& Pair : Map)
		{
			Ar << Pair.Key;
			Ar << Pair.Value;
		}
	}
	return true;
}