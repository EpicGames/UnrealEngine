// Copyright Epic Games, Inc. All Rights Reserved.

#include "LevelSequenceBurnIn.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LevelSequenceBurnIn)

ULevelSequenceBurnIn::ULevelSequenceBurnIn( const FObjectInitializer& ObjectInitializer )
	: Super(ObjectInitializer)
{
}

void ULevelSequenceBurnIn::TakeSnapshotsFrom(ALevelSequenceActor& InActor)
{
	LevelSequenceActor = &InActor;
	ULevelSequencePlayer* SequencePlayer = InActor.GetSequencePlayer();
	if (ensure(SequencePlayer))
	{
		SequencePlayer->OnSequenceUpdated().AddUObject(this, &ULevelSequenceBurnIn::OnSequenceUpdated);
		SequencePlayer->TakeFrameSnapshot(FrameInformation);
	}
}

void ULevelSequenceBurnIn::OnSequenceUpdated(const UMovieSceneSequencePlayer& Player, FFrameTime CurrentTime, FFrameTime PreviousTime)
{
	static_cast<const ULevelSequencePlayer&>(Player).TakeFrameSnapshot(FrameInformation);
}

