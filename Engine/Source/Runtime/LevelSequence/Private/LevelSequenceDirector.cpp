// Copyright Epic Games, Inc. All Rights Reserved.

#include "LevelSequenceDirector.h"
#include "Engine/Level.h"
#include "Engine/World.h"
#include "UObject/Stack.h"
#include "MovieSceneObjectBindingID.h"
#include "LevelSequencePlayer.h"
#include "IMovieScenePlayer.h"
#include "Evaluation/MovieSceneEvaluationTemplateInstance.h"
#include "Evaluation/MovieSceneSequenceHierarchy.h"
#include "MovieSceneSequence.h"
#include "EntitySystem/MovieSceneEntitySystemLinker.h"
#include "EntitySystem/MovieSceneSequenceInstance.h"
#include "Evaluation/MovieScenePlayback.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LevelSequenceDirector)

UWorld* ULevelSequenceDirector::GetWorld() const
{
	if (ULevel* OuterLevel = GetTypedOuter<ULevel>())
	{
		return OuterLevel->OwningWorld;
	}
	return GetTypedOuter<UWorld>();
}

const UE::MovieScene::FSequenceInstance* ULevelSequenceDirector::FindSequenceInstance() const
{
	using namespace UE::MovieScene;

	if (UMovieSceneEntitySystemLinker* Linker = WeakLinker.Get())
	{
		const FInstanceHandle InstanceHandle(InstanceID, InstanceSerial);
		const FInstanceRegistry* InstanceRegistry = Linker->GetInstanceRegistry();
		if (InstanceRegistry->IsHandleValid(InstanceHandle))
		{
			const FSequenceInstance& Instance = InstanceRegistry->GetInstance(InstanceHandle);
			return &Instance;
		}
	}
	return nullptr;
}

UMovieSceneClock* ULevelSequenceDirector::GetRootSequenceCustomClock() const
{
	using namespace UE::MovieScene;

	if (const FSequenceInstance* Instance = FindSequenceInstance())
	{
		TSharedPtr<const FSharedPlaybackState> SharedPlaybackState = Instance->GetSharedPlaybackState();
		UMovieSceneSequence* RootSequence = SharedPlaybackState->GetRootSequence();
		if (RootSequence)
		{
			if (UMovieScene* MovieScene = RootSequence->GetMovieScene())
			{
				return MovieScene->GetCustomClock();
			}
		}
	}
	return nullptr;
}

UMovieSceneClock* ULevelSequenceDirector::GetSequenceCustomClock() const
{
	using namespace UE::MovieScene;

	if (const FSequenceInstance* Instance = FindSequenceInstance())
	{
		TSharedPtr<const FSharedPlaybackState> SharedPlaybackState = Instance->GetSharedPlaybackState();
		UMovieSceneSequence* Sequence = SharedPlaybackState->GetRootSequence();

		const FMovieSceneSequenceID ActualSubSequenceID(SubSequenceID);
		if (ActualSubSequenceID != MovieSceneSequenceID::Root)
		{
			Sequence = SharedPlaybackState->GetHierarchy()->FindSubSequence(ActualSubSequenceID);
		}

		if (Sequence)
		{
			if (UMovieScene* MovieScene = Sequence->GetMovieScene())
			{
				return MovieScene->GetCustomClock();
			}
		}
	}
	return nullptr;
}

FQualifiedFrameTime ULevelSequenceDirector::GetRootSequenceTime() const
{
	using namespace UE::MovieScene;

	if (const FSequenceInstance* Instance = FindSequenceInstance())
	{
		TSharedPtr<const FSharedPlaybackState> SharedPlaybackState = Instance->GetSharedPlaybackState();
		UMovieSceneSequence* RootSequence = SharedPlaybackState->GetRootSequence();
		if (RootSequence)
		{
			const FRootInstanceHandle RootInstanceHandle = Instance->GetRootInstanceHandle();
			const FInstanceRegistry* InstanceRegistry = SharedPlaybackState->GetLinker()->GetInstanceRegistry();
			const FSequenceInstance& RootInstance = InstanceRegistry->GetInstance(RootInstanceHandle);

			// Put the qualified frame time into 'display' rate
			FFrameRate DisplayRate = RootSequence->GetMovieScene()->GetDisplayRate();
			FMovieSceneContext Context = RootInstance.GetContext();

			FFrameTime DisplayRateTime = ConvertFrameTime(Context.GetTime(), Context.GetFrameRate(), DisplayRate);
			return FQualifiedFrameTime(DisplayRateTime, DisplayRate);
		}
	}
	return FQualifiedFrameTime(0, FFrameRate());
}

FQualifiedFrameTime ULevelSequenceDirector::GetCurrentTime() const
{
	using namespace UE::MovieScene;

	if (const FSequenceInstance* Instance = FindSequenceInstance())
	{
		TSharedPtr<const FSharedPlaybackState> SharedPlaybackState = Instance->GetSharedPlaybackState();
		UMovieSceneSequence* Sequence = SharedPlaybackState->GetRootSequence();

		const FMovieSceneSequenceID ActualSubSequenceID(SubSequenceID);
		if (ActualSubSequenceID != MovieSceneSequenceID::Root)
		{
			Sequence = SharedPlaybackState->GetHierarchy()->FindSubSequence(ActualSubSequenceID);
		}

		// Put the qualified frame time into 'display' rate
		FFrameRate DisplayRate = Sequence->GetMovieScene()->GetDisplayRate();
		FMovieSceneContext Context = Instance->GetContext();

		FFrameTime DisplayRateTime = ConvertFrameTime(Context.GetTime(), Context.GetFrameRate(), DisplayRate);
		return FQualifiedFrameTime(DisplayRateTime, DisplayRate);
	}
	return FQualifiedFrameTime(0, FFrameRate());
}


TArray<UObject*> ULevelSequenceDirector::GetBoundObjects(FMovieSceneObjectBindingID ObjectBinding)
{
	using namespace UE::MovieScene;

	TArray<UObject*> Objects;

	if (const FSequenceInstance* Instance = FindSequenceInstance())
	{
		FMovieSceneSequenceID ActualSubSequenceID(SubSequenceID);
		TSharedPtr<const FSharedPlaybackState> SharedPlaybackState = Instance->GetSharedPlaybackState();
		for (TWeakObjectPtr<> WeakObject : ObjectBinding.ResolveBoundObjects(ActualSubSequenceID, SharedPlaybackState.ToSharedRef()))
		{
			if (UObject* Object = WeakObject.Get())
			{
				Objects.Add(Object);
			}
		}
	}
	else
	{
		FFrame::KismetExecutionMessage(TEXT("No player interface available or assigned."), ELogVerbosity::Error);
	}

	return Objects;
}

UObject* ULevelSequenceDirector::GetBoundObject(FMovieSceneObjectBindingID ObjectBinding)
{
	using namespace UE::MovieScene;

	if (const FSequenceInstance* Instance = FindSequenceInstance())
	{
		FMovieSceneSequenceID ActualSubSequenceID(SubSequenceID);
		TSharedPtr<const FSharedPlaybackState> SharedPlaybackState = Instance->GetSharedPlaybackState();
		for (TWeakObjectPtr<> WeakObject : ObjectBinding.ResolveBoundObjects(ActualSubSequenceID, SharedPlaybackState.ToSharedRef()))
		{
			if (UObject* Object = WeakObject.Get())
			{
				return Object;
			}
		}
	}
	else
	{
		FFrame::KismetExecutionMessage(TEXT("No player interface available or assigned."), ELogVerbosity::Error);
	}

	return nullptr;
}

TArray<AActor*> ULevelSequenceDirector::GetBoundActors(FMovieSceneObjectBindingID ObjectBinding)
{
	using namespace UE::MovieScene;

	TArray<AActor*> Actors;

	if (const FSequenceInstance* Instance = FindSequenceInstance())
	{
		FMovieSceneSequenceID ActualSubSequenceID(SubSequenceID);
		TSharedPtr<const FSharedPlaybackState> SharedPlaybackState = Instance->GetSharedPlaybackState();
		for (TWeakObjectPtr<> WeakObject : ObjectBinding.ResolveBoundObjects(ActualSubSequenceID, SharedPlaybackState.ToSharedRef()))
		{
			if (AActor* Actor = Cast<AActor>(WeakObject.Get()))
			{
				Actors.Add(Actor);
			}
		}
	}
	else
	{
		FFrame::KismetExecutionMessage(TEXT("No player interface available or assigned."), ELogVerbosity::Error);
	}

	return Actors;
}

AActor* ULevelSequenceDirector::GetBoundActor(FMovieSceneObjectBindingID ObjectBinding)
{
	using namespace UE::MovieScene;

	if (const FSequenceInstance* Instance = FindSequenceInstance())
	{
		FMovieSceneSequenceID ActualSubSequenceID(SubSequenceID);
		TSharedPtr<const FSharedPlaybackState> SharedPlaybackState = Instance->GetSharedPlaybackState();
		for (TWeakObjectPtr<> WeakObject : ObjectBinding.ResolveBoundObjects(ActualSubSequenceID, SharedPlaybackState.ToSharedRef()))
		{
			if (AActor* Actor = Cast<AActor>(WeakObject.Get()))
			{
				return Actor;
			}
		}
	}
	else
	{
		FFrame::KismetExecutionMessage(TEXT("No player interface available or assigned."), ELogVerbosity::Error);
	}

	return nullptr;
}

UMovieSceneSequence* ULevelSequenceDirector::GetSequence()
{
	using namespace UE::MovieScene;

	if (const FSequenceInstance* Instance = FindSequenceInstance())
	{
		FMovieSceneSequenceID ActualSubSequenceID(SubSequenceID);
		TSharedPtr<const FSharedPlaybackState> SharedPlaybackState = Instance->GetSharedPlaybackState();
		if (ActualSubSequenceID == MovieSceneSequenceID::Root)
		{
			return SharedPlaybackState->GetRootSequence();
		}
		else
		{
			return SharedPlaybackState->GetHierarchy()->FindSubSequence(ActualSubSequenceID);
		}
	}
	else
	{		
		FFrame::KismetExecutionMessage(TEXT("No sequence player."), ELogVerbosity::Error);

		return nullptr;
	}
}


