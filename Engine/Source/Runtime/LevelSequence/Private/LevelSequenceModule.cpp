// Copyright Epic Games, Inc. All Rights Reserved.

#include "LevelSequenceModule.h"
#include "Engine/Level.h"
#include "Modules/ModuleManager.h"
#include "LevelSequence.h"
#include "LevelSequenceActor.h"
#include "LevelSequenceActorSpawner.h"
#include "LevelSequencePlayer.h"
#include "MetaData/MovieSceneShotMetaData.h"
#include "MovieSceneMetaData.h"
#include "UObject/UObjectBaseUtility.h"
#include "IUniversalObjectLocatorModule.h"
#include "UniversalObjectLocatorFragmentType.h"
#include "LegacyLazyObjectPtrFragment.h"

DEFINE_LOG_CATEGORY(LogLevelSequence);

void FLevelSequenceModule::StartupModule()
{
	using namespace UE::UniversalObjectLocator;

	IUniversalObjectLocatorModule& UOLModule = FModuleManager::Get().LoadModuleChecked<IUniversalObjectLocatorModule>("UniversalObjectLocator");

	FFragmentTypeParameters Parameters("ls_lazy_obj_ptr", FText());
	FLegacyLazyObjectPtrFragment::FragmentType = UOLModule.RegisterFragmentType<FLegacyLazyObjectPtrFragment>(Parameters);

	OnCreateMovieSceneObjectSpawnerDelegateHandle = RegisterObjectSpawner(FOnCreateMovieSceneObjectSpawner::CreateStatic(&FLevelSequenceActorSpawner::CreateObjectSpawner));

#if WITH_EDITOR
	// Add empty movie scene meta data to the ULevelSequence CDO to ensure that
	// asset registry tooltips show up in the editor
	UMovieSceneMetaData* MetaData = GetMutableDefault<ULevelSequence>()->FindOrAddMetaData<UMovieSceneMetaData>();
	MetaData->SetFlags(RF_Transient);
	UMovieSceneShotMetaData* ShotMetaData = GetMutableDefault<ULevelSequence>()->FindOrAddMetaData<UMovieSceneShotMetaData>();
	ShotMetaData->SetFlags(RF_Transient);

	LevelSequenceCDO = GetMutableDefault<ULevelSequence>();
#endif
}

void FLevelSequenceModule::ShutdownModule()
{
#if WITH_EDITOR
	if (ULevelSequence* CDO = LevelSequenceCDO.Get())
	{
		CDO->RemoveMetaData<UMovieSceneShotMetaData>();
		CDO->RemoveMetaData<UMovieSceneMetaData>();
	}
#endif

	UnregisterObjectSpawner(OnCreateMovieSceneObjectSpawnerDelegateHandle);
}

namespace
{

static AActor* FindActorBySequenceName(const FString& SequenceNameStr, UWorld* InWorld)
{
	for (ULevel* Level : InWorld->GetLevels())
	{
		if (Level)
		{
			for (AActor* Actor : Level->Actors)
			{
				if (ALevelSequenceActor* LevelSequenceActor = Cast<ALevelSequenceActor>(Actor))
				{
					ULevelSequence* LevelSequence = LevelSequenceActor->GetSequence();
					if (LevelSequence && GetNameSafe(LevelSequence) == SequenceNameStr)
					{
						return Actor;
					}
				}
			}
		}
	}

	return nullptr;
}

TArray<TWeakObjectPtr<UMovieSceneSequencePlayer>> GetLevelSequencePlayers(UWorld* InWorld, const TCHAR** InStr, FOutputDevice& Ar)
{
	TArray<TWeakObjectPtr<UMovieSceneSequencePlayer>> LevelSequencePlayers;

	FString SequencesString = FParse::Token(*InStr, 0);
	TArray<FString> Splits;
	SequencesString.ParseIntoArray(Splits, TEXT(","));

	for (FString Split : Splits)
	{
		AActor* FoundActor = FindActorBySequenceName(Split, InWorld);
		if (ALevelSequenceActor* LevelSequenceActor = Cast<ALevelSequenceActor>(FoundActor))
		{
			UMovieSceneSequencePlayer* Player = LevelSequenceActor->GetSequencePlayer();
			if (!Player)
			{
				LevelSequenceActor->InitializePlayer();
			}

			if (Player)
			{
				LevelSequencePlayers.Add(Player);
			}
		}
	}

	return LevelSequencePlayers;
}


bool HandleLevelSequencePlay(UWorld* InWorld, const TCHAR** InStr, FOutputDevice& Ar)
{
	TArray<TWeakObjectPtr<UMovieSceneSequencePlayer> > LevelSequencePlayers = GetLevelSequencePlayers(InWorld, InStr, Ar);
	if (LevelSequencePlayers.Num() == 0)
	{
		Ar.Log(TEXT("No level sequences found."));
		return false;
	}

	for (TWeakObjectPtr<UMovieSceneSequencePlayer> LevelSequencePlayer : LevelSequencePlayers)
	{
		if (LevelSequencePlayer.Get())
		{
			LevelSequencePlayer.Get()->Play();
		}
	}
	return true;
}


bool HandleLevelSequencePause(UWorld* InWorld, const TCHAR** InStr, FOutputDevice& Ar)
{
	TArray<TWeakObjectPtr<UMovieSceneSequencePlayer> > LevelSequencePlayers = GetLevelSequencePlayers(InWorld, InStr, Ar);
	if (LevelSequencePlayers.Num() == 0)
	{
		Ar.Log(TEXT("No level sequences found."));
		return false;
	}

	for (TWeakObjectPtr<UMovieSceneSequencePlayer> LevelSequencePlayer : LevelSequencePlayers)
	{
		if (LevelSequencePlayer.Get())
		{
			LevelSequencePlayer.Get()->Pause();
		}
	}
	return true;
}


bool HandleLevelSequenceStop(UWorld* InWorld, const TCHAR** InStr, FOutputDevice& Ar)
{
	TArray<TWeakObjectPtr<UMovieSceneSequencePlayer> > LevelSequencePlayers = GetLevelSequencePlayers(InWorld, InStr, Ar);
	if (LevelSequencePlayers.Num() == 0)
	{
		Ar.Log(TEXT("No level sequences found."));
		return false;
	}

	for (TWeakObjectPtr<UMovieSceneSequencePlayer> LevelSequencePlayer : LevelSequencePlayers)
	{
		if (LevelSequencePlayer.Get())
		{
			LevelSequencePlayer.Get()->Stop();
		}
	}
	return true;
}


bool HandleLevelSequenceSetPlaybackPosition(UWorld* InWorld, const TCHAR** InStr, FOutputDevice& Ar)
{
	TArray<TWeakObjectPtr<UMovieSceneSequencePlayer> > LevelSequencePlayers = GetLevelSequencePlayers(InWorld, InStr, Ar);
	if (LevelSequencePlayers.Num() == 0)
	{
		Ar.Log(TEXT("No level sequences found."));
		return false;
	}

	FString FrameNumberString = FParse::Token(*InStr, 0);
	if (!FChar::IsDigit(**FrameNumberString))
	{
		Ar.Logf(TEXT("Invalid frame number to set playback position to: %s"), *FrameNumberString);
		return false;
	}

	int32 FrameNumber = FCString::Atoi(*FrameNumberString);
	Ar.Logf(TEXT("Setting playback position to: %d"), FrameNumber);

	for (TWeakObjectPtr<UMovieSceneSequencePlayer> LevelSequencePlayer : LevelSequencePlayers)
	{
		if (LevelSequencePlayer.Get())
		{
			FMovieSceneSequencePlaybackParams PlaybackParams(FFrameTime(FrameNumber), EUpdatePositionMethod::Play);
			LevelSequencePlayer.Get()->SetPlaybackPosition(PlaybackParams);
		}
	}
	return true;
}

bool HandleLevelSequencePlayTo(UWorld* InWorld, const TCHAR** InStr, FOutputDevice& Ar)
{
	TArray<TWeakObjectPtr<UMovieSceneSequencePlayer> > LevelSequencePlayers = GetLevelSequencePlayers(InWorld, InStr, Ar);
	if (LevelSequencePlayers.Num() == 0)
	{
		Ar.Log(TEXT("No level sequences found."));
		return false;
	}

	FString FrameNumberString = FParse::Token(*InStr, 0);
	if (!FChar::IsDigit(**FrameNumberString))
	{
		Ar.Logf(TEXT("Invalid frame number to play to: %s"), *FrameNumberString);
		return false;
	}

	int32 FrameNumber = FCString::Atoi(*FrameNumberString);
	Ar.Logf(TEXT("Playing to: %d"), FrameNumber);

	for (TWeakObjectPtr<UMovieSceneSequencePlayer> LevelSequencePlayer : LevelSequencePlayers)
	{
		if (LevelSequencePlayer.Get())
		{
			FMovieSceneSequencePlaybackParams PlaybackParams(FFrameTime(FrameNumber), EUpdatePositionMethod::Play);
			FMovieSceneSequencePlayToParams PlayToParams;
			LevelSequencePlayer.Get()->PlayTo(PlaybackParams, PlayToParams);
		}
	}
	return true;
}

bool HandleLevelSequenceSetClockSource(UWorld* InWorld, const TCHAR** InStr, FOutputDevice& Ar)
{
	TArray<TWeakObjectPtr<UMovieSceneSequencePlayer> > LevelSequencePlayers = GetLevelSequencePlayers(InWorld, InStr, Ar);
	if (LevelSequencePlayers.Num() == 0)
	{
		Ar.Log(TEXT("No level sequences found."));
		return false;
	}

	FString ClockSource = FParse::Token(*InStr, 0);

	TSharedPtr<FMovieSceneTimeController> TimeController;

	if (ClockSource.Compare("Tick", ESearchCase::IgnoreCase) == 0)
	{
		TimeController = MakeShared<FMovieSceneTimeController_Tick>();
	}
	else if (ClockSource.Compare("Audio", ESearchCase::IgnoreCase) == 0)
	{
		TimeController = MakeShared<FMovieSceneTimeController_AudioClock>();
	}
	else if (ClockSource.Compare("Platform", ESearchCase::IgnoreCase) == 0)
	{
		TimeController = MakeShared<FMovieSceneTimeController_PlatformClock>();
	}
	else if (ClockSource.Compare("RelativeTimecode", ESearchCase::IgnoreCase) == 0)
	{
		TimeController = MakeShared<FMovieSceneTimeController_RelativeTimecodeClock>();
	}
	else if (ClockSource.Compare("Timecode", ESearchCase::IgnoreCase) == 0)
	{
		TimeController = MakeShared<FMovieSceneTimeController_TimecodeClock>();
	}
	else if (ClockSource.Compare("PlayEveryFrame", ESearchCase::IgnoreCase) == 0)
	{
		TimeController = MakeShared<FMovieSceneTimeController_PlayEveryFrame>();
	}
	else
	{
		Ar.Log(TEXT("Unknown clock source. Valid clock sources are: Tick, Audio, Platform, RelativeTimecode, Timecode, PlayEveryFrame"));
		return false;
	}

	for (TWeakObjectPtr<UMovieSceneSequencePlayer> LevelSequencePlayer : LevelSequencePlayers)
	{
		if (LevelSequencePlayer.Get())
		{
			LevelSequencePlayer.Get()->SetTimeController(TimeController);
		}
	}
	return true;
}

} // empty namespace

bool FLevelSequenceModule::Exec_Runtime(class UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar)
{
	if (FParse::Command(&Cmd, TEXT("Sequencer")))
	{
		if (FParse::Command(&Cmd, TEXT("Play")))
		{
			if (HandleLevelSequencePlay(InWorld, &Cmd, Ar))
			{
				return true;
			}
		}
		else if (FParse::Command(&Cmd, TEXT("Pause")))
		{
			if (HandleLevelSequencePause(InWorld, &Cmd, Ar))
			{
				return true;
			}
		}
		else if (FParse::Command(&Cmd, TEXT("Stop")))
		{
			if (HandleLevelSequenceStop(InWorld, &Cmd, Ar))
			{
				return true;
			}
		}
		else if (FParse::Command(&Cmd, TEXT("SetPlaybackPosition")))
		{
			if (HandleLevelSequenceSetPlaybackPosition(InWorld, &Cmd, Ar))
			{
				return true;
			}
		}
		else if (FParse::Command(&Cmd, TEXT("PlayTo")))
		{
			if (HandleLevelSequencePlayTo(InWorld, &Cmd, Ar))
			{
				return true;
			}
		}
		else if (FParse::Command(&Cmd, TEXT("SetClockSource")))
		{
			if (HandleLevelSequenceSetClockSource(InWorld, &Cmd, Ar))
			{
				return true;
			}
		}

		// show usage
		Ar.Log(TEXT("Usage: Sequencer <Command>"));
		Ar.Log(TEXT(""));
		Ar.Log(TEXT("Command"));
		Ar.Log(TEXT("    Play <SequenceName> = Start playback forwards from the current time cursor position, using the current play rate"));
		Ar.Log(TEXT("    Pause <SequenceName> = Pause playback"));
		Ar.Log(TEXT("    Stop <SequenceName> = Stop playback and move the cursor to the end (or start, for reversed playback) of the sequence"));
		Ar.Log(TEXT("    SetPlaybackPosition <SequenceName> <FrameNumber> = Set the current time of the player by evaluating from the current time to the specified time"));
		Ar.Log(TEXT("    PlayTo <SequenceName> <FrameNumber> = Play from the current position to the requested position and pause"));
		Ar.Log(TEXT("    SetClockSource <SequenceName> <ClockSource, ie. Tick, Audio, PlayEveryFrame> = Set the clock source"));
	}
		
	return false;
}

FDelegateHandle FLevelSequenceModule::RegisterObjectSpawner(FOnCreateMovieSceneObjectSpawner InOnCreateMovieSceneObjectSpawner)
{
	OnCreateMovieSceneObjectSpawnerDelegates.Add(InOnCreateMovieSceneObjectSpawner);
	return OnCreateMovieSceneObjectSpawnerDelegates.Last().GetHandle();
}

void FLevelSequenceModule::UnregisterObjectSpawner(FDelegateHandle InHandle) 
{
	OnCreateMovieSceneObjectSpawnerDelegates.RemoveAll([=](const FOnCreateMovieSceneObjectSpawner& Delegate) { return Delegate.GetHandle() == InHandle; });
}

FLevelSequenceModule::FOnNewActorTrackAdded& FLevelSequenceModule::OnNewActorTrackAdded()
{
	return NewActorTrackAdded;
}

void FLevelSequenceModule::GenerateObjectSpawners(TArray<TSharedRef<IMovieSceneObjectSpawner>>& OutSpawners) const
{
	for (const FOnCreateMovieSceneObjectSpawner& SpawnerFactory : OnCreateMovieSceneObjectSpawnerDelegates)
	{
		check(SpawnerFactory.IsBound());
		OutSpawners.Add(SpawnerFactory.Execute());
	}

	// Now sort the spawners. Editor spawners should come first so they override runtime versions of the same supported type in-editor.
	// @TODO: we could also sort by most-derived type here to allow for type specific behaviors
	OutSpawners.Sort([](TSharedRef<IMovieSceneObjectSpawner> LHS, TSharedRef<IMovieSceneObjectSpawner> RHS)
	{
		return (LHS->IsEditor() > RHS->IsEditor()) || ( (LHS->IsEditor() == RHS->IsEditor()) && (LHS->GetSpawnerPriority() > RHS->GetSpawnerPriority()));
	});
}

IMPLEMENT_MODULE(FLevelSequenceModule, LevelSequence);
