// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/CoreMisc.h"
#include "ILevelSequenceModule.h"

DECLARE_LOG_CATEGORY_EXTERN(LogLevelSequence, Log, All);

class ULevelSequence;

/**
 * Implements the LevelSequence module.
 */
class FLevelSequenceModule : public ILevelSequenceModule, public FSelfRegisteringExec
{
public:

	// IModuleInterface interface
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;

	// ILevelSequenceModule interface
	virtual FDelegateHandle RegisterObjectSpawner(FOnCreateMovieSceneObjectSpawner InOnCreateMovieSceneObjectSpawner) override;
	virtual void GenerateObjectSpawners(TArray<TSharedRef<IMovieSceneObjectSpawner>>& OutSpawners) const override;
	virtual void UnregisterObjectSpawner(FDelegateHandle InHandle) override;
	virtual FOnNewActorTrackAdded& OnNewActorTrackAdded() override;

protected:
	// FSelfRegisteringExec interface
	virtual bool Exec_Runtime(class UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar) override;

public:
	/** List of object spawner delegates used to extend the spawn register */
	TArray< FOnCreateMovieSceneObjectSpawner > OnCreateMovieSceneObjectSpawnerDelegates;

	/** Internal delegate handle used for spawning actors */
	FDelegateHandle OnCreateMovieSceneObjectSpawnerDelegateHandle;

private:
	// Weak ptr to the level sequence CDO so we can gracefully remove the meta-data on shutdown module
	// without crashing when ShutdownModule is called after the CDO has been destroyed.
	TWeakObjectPtr<ULevelSequence> LevelSequenceCDO;
	FOnNewActorTrackAdded NewActorTrackAdded;
};
