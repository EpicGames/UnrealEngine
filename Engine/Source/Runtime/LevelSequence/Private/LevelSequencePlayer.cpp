// Copyright Epic Games, Inc. All Rights Reserved.

#include "LevelSequencePlayer.h"
#include "Engine/GameInstance.h"
#include "GameFramework/Actor.h"
#include "MovieScene.h"
#include "Misc/CoreDelegates.h"
#include "EngineGlobals.h"
#include "Engine/Level.h"
#include "Camera/PlayerCameraManager.h"
#include "UObject/Package.h"
#include "GameFramework/PlayerController.h"
#include "Camera/CameraComponent.h"
#include "Tickable.h"
#include "Engine/LevelScriptActor.h"
#include "MovieSceneCommonHelpers.h"
#include "Sections/MovieSceneSubSection.h"
#include "LevelSequenceSpawnRegister.h"
#include "Engine/Engine.h"
#include "Engine/LevelStreaming.h"
#include "Engine/LocalPlayer.h"
#include "Tracks/MovieSceneCinematicShotTrack.h"
#include "Sections/MovieSceneCinematicShotSection.h"
#include "Systems/MovieSceneMotionVectorSimulationSystem.h"
#include "EntitySystem/MovieSceneEntitySystemLinker.h"
#include "EntitySystem/MovieSceneSharedPlaybackState.h"
#include "LevelSequenceActor.h"
#include "Modules/ModuleManager.h"
#include "LevelUtils.h"
#include "ProfilingDebugging/CsvProfiler.h"
#include "LevelSequenceModule.h"
#include "Generators/MovieSceneEasingCurves.h"
#include "UniversalObjectLocatorResolveParams.h"
#include "UniversalObjectLocators/ActorLocatorFragment.h"
#include "UniversalObjectLocatorResolveParameterBuffer.inl"
#include "Evaluation/CameraCutPlaybackCapability.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LevelSequencePlayer)

/* ULevelSequencePlayer structors
 *****************************************************************************/

ULevelSequencePlayer::ULevelSequencePlayer(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{}


/* ULevelSequencePlayer interface
 *****************************************************************************/

ULevelSequencePlayer* ULevelSequencePlayer::CreateLevelSequencePlayer(UObject* WorldContextObject, ULevelSequence* InLevelSequence, FMovieSceneSequencePlaybackSettings Settings, ALevelSequenceActor*& OutActor)
{
	if (InLevelSequence == nullptr)
	{
		return nullptr;
	}

	UWorld* World = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull);
	if (World == nullptr || World->bIsTearingDown)
	{
		return nullptr;
	}

	FActorSpawnParameters SpawnParams;
	SpawnParams.SpawnCollisionHandlingOverride = ESpawnActorCollisionHandlingMethod::AlwaysSpawn;
	SpawnParams.ObjectFlags |= RF_Transient;
	SpawnParams.bAllowDuringConstructionScript = true;

	// Defer construction for autoplay so that BeginPlay() is called
	SpawnParams.bDeferConstruction = true;

	ALevelSequenceActor* Actor = World->SpawnActor<ALevelSequenceActor>(SpawnParams);

	Actor->PlaybackSettings = Settings;
	Actor->GetSequencePlayer()->SetPlaybackSettings(Settings);

	Actor->SetSequence(InLevelSequence);

	Actor->InitializePlayer();
	OutActor = Actor;

	FTransform DefaultTransform;
	Actor->FinishSpawning(DefaultTransform);

	return Actor->GetSequencePlayer();
}

/* ULevelSequencePlayer implementation
 *****************************************************************************/

void ULevelSequencePlayer::Initialize(ULevelSequence* InLevelSequence, ULevel* InLevel, const FLevelSequenceCameraSettings& InCameraSettings)
{
	using namespace UE::MovieScene;

	World = InLevel->OwningWorld;
	Level = InLevel;
	CameraSettings = InCameraSettings;

	UMovieSceneSequencePlayer::Initialize(InLevelSequence);

	// The parent player class' root evaluation template may or may not have re-initialized itself.
	// For instance, if we are given the same sequence asset we already had before, and nothing else
	// (such as playback context) has changed, no actual re-initialization occurs and we keep the
	// same shared playback state as before.
	// That state would already have the spawn register and camera cut capabilies... however, our 
	// spawn register was just re-created (see a few lines above) so we need to overwrite the
	// capability pointer to the new object.
	InitializeLevelSequenceRootInstance(RootTemplateInstance.GetSharedPlaybackState().ToSharedRef());
}

void ULevelSequencePlayer::SetSourceActorContext(UWorld* InStreamingWorld, FActorContainerID InContainerID, FTopLevelAssetPath InSourceAssetPath)
{
	WeakStreamingWorld = InStreamingWorld;
	ContainerID = InContainerID;
	SourceAssetPath = InSourceAssetPath;
}

void ULevelSequencePlayer::ResolveBoundObjects(UE::UniversalObjectLocator::FResolveParams& LocatorResolveParams, const FGuid& InBindingId, FMovieSceneSequenceID SequenceID, UMovieSceneSequence& InSequence, TArray<UObject*, TInlineAllocator<1>>& OutObjects) const
{
	using namespace UE::UniversalObjectLocator;
	using namespace UE::MovieScene;

	bool bAllowDefault = PlaybackClient ? PlaybackClient->RetrieveBindingOverrides(InBindingId, SequenceID, OutObjects) : true;

	if (bAllowDefault)
	{
		if (const FMovieSceneBindingReferences* BindingReferences = InSequence.GetBindingReferences())
		{
			FMovieSceneBindingResolveParams BindingResolveParams{ &InSequence, InBindingId, SequenceID, LocatorResolveParams.Context };
			if (LocatorResolveParams.ParameterBuffer == nullptr)
			{
				// Allocate temporary local buffer for this
				TInlineResolveParameterBuffer<128> Buffer;
				LocatorResolveParams.ParameterBuffer = &Buffer;
				LocatorResolveParams.ParameterBuffer->AddParameter(FActorLocatorFragmentResolveParameter::ParameterType, WeakStreamingWorld.Get(), ContainerID, SourceAssetPath);
				BindingReferences->ResolveBinding(BindingResolveParams, LocatorResolveParams, ConstCastSharedPtr<const UE::MovieScene::FSharedPlaybackState>(const_cast<ULevelSequencePlayer*>(this)->FindSharedPlaybackState()), OutObjects);
				LocatorResolveParams.ParameterBuffer = nullptr;
			}
			else
			{
				LocatorResolveParams.ParameterBuffer->AddParameter(FActorLocatorFragmentResolveParameter::ParameterType, WeakStreamingWorld.Get(), ContainerID, SourceAssetPath);
				BindingReferences->ResolveBinding(BindingResolveParams, LocatorResolveParams, ConstCastSharedPtr<const UE::MovieScene::FSharedPlaybackState>(const_cast<ULevelSequencePlayer*>(this)->FindSharedPlaybackState()), OutObjects);
			}
		}
		else
		{
			if (LocatorResolveParams.ParameterBuffer == nullptr)
			{
				// Allocate temporary local buffer for this
				TInlineResolveParameterBuffer<128> Buffer;
				LocatorResolveParams.ParameterBuffer = &Buffer;
				LocatorResolveParams.ParameterBuffer->AddParameter(FActorLocatorFragmentResolveParameter::ParameterType, WeakStreamingWorld.Get(), ContainerID, SourceAssetPath);
				InSequence.LocateBoundObjects(InBindingId, LocatorResolveParams, GetSharedPlaybackState(), OutObjects);
				LocatorResolveParams.ParameterBuffer = nullptr;
			}
			else
			{
				LocatorResolveParams.ParameterBuffer->AddParameter(FActorLocatorFragmentResolveParameter::ParameterType, WeakStreamingWorld.Get(), ContainerID, SourceAssetPath);
				InSequence.LocateBoundObjects(InBindingId, LocatorResolveParams, GetSharedPlaybackState(), OutObjects);
			}
		}
	}
}

bool ULevelSequencePlayer::CanPlay() const
{
	return World.IsValid();
}

void ULevelSequencePlayer::OnStartedPlaying()
{
	EnableCinematicMode(true);
}

void ULevelSequencePlayer::OnStopped()
{
	EnableCinematicMode(false);
}

void ULevelSequencePlayer::UpdateMovieSceneInstance(FMovieSceneEvaluationRange InRange, EMovieScenePlayerStatus::Type PlayerStatus, const FMovieSceneUpdateArgs& Args)
{
	UMovieSceneSequencePlayer::UpdateMovieSceneInstance(InRange, PlayerStatus, Args);

	// TODO-ludovic: we should move this to a post-evaluation callback when the evaluation is asynchronous.
	FLevelSequencePlayerSnapshot NewSnapshot;
	TakeFrameSnapshot(NewSnapshot);

	if (!PreviousSnapshot.IsSet() || PreviousSnapshot.GetValue().CurrentShotName != NewSnapshot.CurrentShotName)
	{
		CSV_EVENT_GLOBAL(TEXT("%s"), *NewSnapshot.CurrentShotName);
		//UE_LOGF(LogMovieScene, Log, "Shot evaluated: '%ls'", *NewSnapshot.CurrentShotName);
	}

	PreviousSnapshot = NewSnapshot;
}

/* FCameraCutPlaybackCapability interface
 *****************************************************************************/

bool ULevelSequencePlayer::ShouldUpdateCameraCut()
{
	return !PlaybackSettings.bDisableCameraCuts;
}

float ULevelSequencePlayer::GetCameraBlendPlayRate()
{
	return PlaybackSettings.PlayRate;
}

TOptional<EAspectRatioAxisConstraint> ULevelSequencePlayer::GetAspectRatioAxisConstraintOverride()
{
	return CameraSettings.bOverrideAspectRatioAxisConstraint ?
		TOptional<EAspectRatioAxisConstraint>(CameraSettings.AspectRatioAxisConstraint) :
		TOptional<EAspectRatioAxisConstraint>();
}

void ULevelSequencePlayer::OnCameraCutUpdated(const UE::MovieScene::FOnCameraCutUpdatedParams& Params)
{
	CachedCameraComponent = Params.ViewTargetCamera;

	if (OnCameraCut.IsBound())
	{
		OnCameraCut.Broadcast(Params.ViewTargetCamera);
	}
}

/* IMovieScenePlayer interface
 *****************************************************************************/

UObject* ULevelSequencePlayer::GetPlaybackContext() const
{
	if (ALevelSequenceActor* LevelSequenceActor = GetTypedOuter<ALevelSequenceActor>())
	{
		return LevelSequenceActor;
	}
	return World.Get();
}

TArray<UObject*> ULevelSequencePlayer::GetEventContexts() const
{
	TArray<UObject*> EventContexts;
	if (World.IsValid())
	{
		GetEventContexts(*World, EventContexts);
	}

	return EventContexts;
}

void ULevelSequencePlayer::GetEventContexts(UWorld& InWorld, TArray<UObject*>& OutContexts)
{
	if (InWorld.GetLevelScriptActor())
	{
		OutContexts.Add(InWorld.GetLevelScriptActor());
	}

	for (ULevelStreaming* StreamingLevel : InWorld.GetStreamingLevels())
	{
		if (StreamingLevel && StreamingLevel->GetLevelScriptActor())
		{
			OutContexts.Add(StreamingLevel->GetLevelScriptActor());
		}
	}
}

void ULevelSequencePlayer::InitializeRootInstance(TSharedRef<UE::MovieScene::FSharedPlaybackState> NewSharedPlaybackState)
{
	using namespace UE::MovieScene;

	SpawnRegister = MakeShareable(new FLevelSequenceSpawnRegister);

	Super::InitializeRootInstance(NewSharedPlaybackState);

	InitializeLevelSequenceRootInstance(NewSharedPlaybackState);
}

void ULevelSequencePlayer::InitializeLevelSequenceRootInstance(TSharedRef<UE::MovieScene::FSharedPlaybackState> NewSharedPlaybackState)
{
	NewSharedPlaybackState->SetOrAddCapabilityRaw<FMovieSceneSpawnRegister>(SpawnRegister.Get());
	NewSharedPlaybackState->SetOrAddCapabilityRaw<FCameraCutPlaybackCapability>((FCameraCutPlaybackCapability*)this);
	NewSharedPlaybackState->SetOrAddCapability<UE::MovieScene::FMovieSceneAudioRecordingCapability>(); // constructs inline in NewSharedPlaybackState
}

void ULevelSequencePlayer::TakeFrameSnapshot(FLevelSequencePlayerSnapshot& OutSnapshot) const
{
	if (!ensure(Sequence))
	{
		return;
	}

	// In Play Rate Resolution
	const FFrameTime StartTimeWithoutWarmupFrames = SnapshotOffsetTime.IsSet() ? StartTime + SnapshotOffsetTime.GetValue() : StartTime;
	const FFrameTime CurrentPlayTime = PlayPosition.GetCurrentPosition();
	// In Playback Resolution
	const FFrameTime CurrentSequenceTime		  = ConvertFrameTime(CurrentPlayTime, PlayPosition.GetInputRate(), PlayPosition.GetOutputRate());

	OutSnapshot.RootTime = FQualifiedFrameTime(CurrentPlayTime, PlayPosition.GetInputRate());
	OutSnapshot.RootName = Sequence->GetName();

	OutSnapshot.CurrentShotName = OutSnapshot.RootName;
	OutSnapshot.CurrentShotLocalTime = FQualifiedFrameTime(CurrentPlayTime, PlayPosition.GetInputRate());
	OutSnapshot.CameraComponent = CachedCameraComponent.IsValid() ? CachedCameraComponent.Get() : nullptr;
	OutSnapshot.ShotID = MovieSceneSequenceID::Invalid;

	OutSnapshot.ActiveShot = Cast<ULevelSequence>(Sequence);

	UMovieScene* MovieScene = Sequence->GetMovieScene();

#if WITH_EDITORONLY_DATA
	OutSnapshot.SourceTimecode = MovieScene->GetEarliestTimecodeSource().Timecode.ToString();
#endif

	UMovieSceneCinematicShotTrack* ShotTrack = MovieScene->FindTrack<UMovieSceneCinematicShotTrack>();
	if (ShotTrack)
	{
		UMovieSceneCinematicShotSection* ActiveShot = nullptr;
		for (UMovieSceneSection* Section : ShotTrack->GetAllSections())
		{
			if (!ensure(Section))
			{
				continue;
			}

			// It's unfortunate that we have to copy the logic of UMovieSceneCinematicShotTrack::GetRowCompilerRules() to some degree here, but there's no better way atm
			bool bThisShotIsActive = Section->IsActive();

			TRange<FFrameNumber> SectionRange = Section->GetRange();
			bThisShotIsActive = bThisShotIsActive && SectionRange.Contains(CurrentSequenceTime.FrameNumber);

			if (bThisShotIsActive && ActiveShot)
			{
				if (Section->GetRowIndex() < ActiveShot->GetRowIndex())
				{
					bThisShotIsActive = true;
				}
				else if (Section->GetRowIndex() == ActiveShot->GetRowIndex())
				{
					// On the same row - latest start wins
					bThisShotIsActive = TRangeBound<FFrameNumber>::MaxLower(SectionRange.GetLowerBound(), ActiveShot->GetRange().GetLowerBound()) == SectionRange.GetLowerBound();
				}
				else
				{
					bThisShotIsActive = false;
				}
			}

			if (bThisShotIsActive)
			{
				ActiveShot = Cast<UMovieSceneCinematicShotSection>(Section);
			}
		}

		if (ActiveShot)
		{
			// Assume that shots with no sequence start at 0.
			FMovieSceneSequenceTransform OuterToInnerTransform = ActiveShot->OuterToInnerTransform();
			UMovieSceneSequence*         InnerSequence = ActiveShot->GetSequence();
			FFrameRate                   InnerTickResoloution = InnerSequence ? InnerSequence->GetMovieScene()->GetTickResolution() : PlayPosition.GetOutputRate();
			FFrameRate                   InnerFrameRate = InnerSequence ? InnerSequence->GetMovieScene()->GetDisplayRate() : PlayPosition.GetInputRate();
			FFrameTime                   InnerDisplayTime = ConvertFrameTime(CurrentSequenceTime * OuterToInnerTransform, InnerTickResoloution, InnerFrameRate);

			OutSnapshot.CurrentShotName = ActiveShot->GetShotDisplayName();
			OutSnapshot.CurrentShotLocalTime = FQualifiedFrameTime(InnerDisplayTime, InnerFrameRate);
			OutSnapshot.ShotID = ActiveShot->GetSequenceID();
			OutSnapshot.ActiveShot = Cast<ULevelSequence>(ActiveShot->GetSequence());

#if WITH_EDITORONLY_DATA
			FFrameNumber  InnerFrameNumber = InnerFrameRate.AsFrameNumber(InnerFrameRate.AsSeconds(InnerDisplayTime));
			FFrameNumber  InnerStartFrameNumber = ActiveShot->TimecodeSource.Timecode.ToFrameNumber(InnerFrameRate);
			FFrameNumber  InnerCurrentFrameNumber = InnerStartFrameNumber + InnerFrameNumber;
			FTimecode     InnerCurrentTimecode = ActiveShot->TimecodeSource.Timecode.FromFrameNumber(InnerCurrentFrameNumber, InnerFrameRate, false);

			OutSnapshot.SourceTimecode = InnerCurrentTimecode.ToString();
#else
			OutSnapshot.SourceTimecode = FTimecode().ToString();
#endif
		}
	}
}

void ULevelSequencePlayer::EnableCinematicMode(bool bEnable)
{
	// iterate through the controller list and set cinematic mode if necessary
	bool bNeedsCinematicMode = PlaybackSettings.bDisableMovementInput || PlaybackSettings.bDisableLookAtInput || PlaybackSettings.bHidePlayer || PlaybackSettings.bHideHud;

	if (bNeedsCinematicMode)
	{
		if (World.IsValid())
		{
			for (FConstPlayerControllerIterator Iterator = World->GetPlayerControllerIterator(); Iterator; ++Iterator)
			{
				APlayerController* PC = Iterator->Get();
				if (PC && PC->IsLocalController())
				{
					PC->SetCinematicMode(bEnable, PlaybackSettings.bHidePlayer, PlaybackSettings.bHideHud, PlaybackSettings.bDisableMovementInput, PlaybackSettings.bDisableLookAtInput);
				}
			}
		}
	}
}

void ULevelSequencePlayer::RewindForReplay()
{
	// Stop the sequence when starting to seek through a replay. This restores our state to be unmodified
	// in case the replay is seeking to before playback. If we're in the middle of playback after rewinding,
	// the replay will feed the correct packets to synchronize our playback time and state.
	Stop();

	NetSyncProps.LastKnownPosition = FFrameTime(0);
	NetSyncProps.LastKnownStatus = EMovieScenePlayerStatus::Stopped;
	NetSyncProps.LastKnownNumLoops = 0;
	NetSyncProps.LastKnownSerialNumber = 0;
}
