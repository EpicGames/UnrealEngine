// Copyright Epic Games, Inc. All Rights Reserved.

#include "LevelSequenceProjectSettings.h"

#include "HAL/IConsoleManager.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LevelSequenceProjectSettings)

ULevelSequenceProjectSettings::ULevelSequenceProjectSettings()
	: bDefaultLockEngineToDisplayRate(false)
	, DefaultDisplayRate("30fps")
	, DefaultTickResolution("24000fps")
	, DefaultClockSource(EUpdateClockSource::Tick)
{ }


void ULevelSequenceProjectSettings::PostInitProperties()
{
	Super::PostInitProperties(); 

#if WITH_EDITOR
	if(IsTemplate())
	{
		// Most classes that uses this console backed cvar to import the cvar values onto the config. 
		// This isn't quite desirable because it means changing a value in the project settings will 
		// get overwritten by cvar (possibly default) values. So, instead of importing console variable 
		// values onto the config, set the console variable values based on the config values.
		// ImportConsoleVariableValues();
	
		if (IConsoleVariable* ConsoleVariable = IConsoleManager::Get().FindConsoleVariable(TEXT("LevelSequence.DefaultLockEngineToDisplayRate")))
		{
			ConsoleVariable->Set((int32)bDefaultLockEngineToDisplayRate, ECVF_SetByProjectSetting);
		}

		if (IConsoleVariable* ConsoleVariable = IConsoleManager::Get().FindConsoleVariable(TEXT("LevelSequence.DefaultTickResolution")))
		{
			ConsoleVariable->Set(*DefaultTickResolution, ECVF_SetByProjectSetting);
		}

		if (IConsoleVariable* ConsoleVariable = IConsoleManager::Get().FindConsoleVariable(TEXT("LevelSequence.DefaultDisplayRate")))
		{
			ConsoleVariable->Set(*DefaultDisplayRate, ECVF_SetByProjectSetting);
		}

		if (IConsoleVariable* ConsoleVariable = IConsoleManager::Get().FindConsoleVariable(TEXT("LevelSequence.DefaultClockSource")))
		{
			ConsoleVariable->Set((int32)DefaultClockSource, ECVF_SetByProjectSetting);
		}
	}
#endif
}

#if WITH_EDITOR

void ULevelSequenceProjectSettings::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	if(PropertyChangedEvent.Property)
	{
		ExportValuesToConsoleVariables(PropertyChangedEvent.Property);
	}
}

#endif


