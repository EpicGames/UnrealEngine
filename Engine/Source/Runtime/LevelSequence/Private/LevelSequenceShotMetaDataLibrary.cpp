// Copyright Epic Games, Inc. All Rights Reserved.

#include "LevelSequenceShotMetaDataLibrary.h"

#include "LevelSequence.h"
#include "MetaData/MovieSceneShotMetaData.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LevelSequenceShotMetaDataLibrary)

namespace UE::Sequencer::TakesMetaDataDetail
{
static UMovieSceneShotMetaData* FindMetaData(const ULevelSequence* InLevelSequence)
{
#if WITH_EDITOR 
	return InLevelSequence ? InLevelSequence->FindMetaData<UMovieSceneShotMetaData>() : nullptr;
#else
	// If needed at runtime, we could consider wrapping UMovieSceneShotMetaData in UAssetUserData during cook time.
	return nullptr;
#endif
}
static UMovieSceneShotMetaData* FindOrAddMetaData(ULevelSequence* InLevelSequence)
{
#if WITH_EDITOR
	return InLevelSequence ? InLevelSequence->FindOrAddMetaData<UMovieSceneShotMetaData>() : nullptr;
#else
	// If needed at runtime, we could consider wrapping UMovieSceneShotMetaData in UAssetUserData during cook time.
	return nullptr;
#endif
}
}

bool ULevelSequenceShotMetaDataLibrary::GetIsNoGood(const ULevelSequence* InLevelSequence, bool& bOutNoGood)
{
	UMovieSceneShotMetaData* MetaData = UE::Sequencer::TakesMetaDataDetail::FindMetaData(InLevelSequence);
	if (!MetaData || !MetaData->GetIsNoGood().IsSet())
	{
		return false;
	}
	
	bOutNoGood = *MetaData->GetIsNoGood();
	return true;
}

bool ULevelSequenceShotMetaDataLibrary::GetIsFlagged(const ULevelSequence* InLevelSequence, bool& bOutIsFlagged)
{
	UMovieSceneShotMetaData* MetaData = UE::Sequencer::TakesMetaDataDetail::FindMetaData(InLevelSequence);
	if (!MetaData || !MetaData->GetIsFlagged().IsSet())
	{
		return false;
	}
	
	bOutIsFlagged = *MetaData->GetIsFlagged();
	return true;
}

bool ULevelSequenceShotMetaDataLibrary::GetIsRecorded(const ULevelSequence* InLevelSequence, bool& bOutIsRecorded)
{
	const UMovieSceneShotMetaData* MetaData = UE::Sequencer::TakesMetaDataDetail::FindMetaData(InLevelSequence);
	if (!MetaData || !MetaData->GetIsRecorded().IsSet())
	{
		return false;
	}
	
	bOutIsRecorded = *MetaData->GetIsRecorded();
	return true;
}

bool ULevelSequenceShotMetaDataLibrary::GetIsSubSequence(const ULevelSequence* InLevelSequence, bool& bOutIsSubSequence)
{
	const UMovieSceneShotMetaData* MetaData = UE::Sequencer::TakesMetaDataDetail::FindMetaData(InLevelSequence);
	if (!MetaData || !MetaData->GetIsSubSequence().IsSet())
	{
		return false;
	}
	
	bOutIsSubSequence = *MetaData->GetIsSubSequence();
	return true;
}

bool ULevelSequenceShotMetaDataLibrary::GetFavoriteRating(const ULevelSequence* InLevelSequence, int32& OutFavoriteRating)
{
	UMovieSceneShotMetaData* MetaData = UE::Sequencer::TakesMetaDataDetail::FindMetaData(InLevelSequence);
	if (!MetaData || !MetaData->GetFavoriteRating().IsSet())
	{
		return false;
	}
	
	OutFavoriteRating = *MetaData->GetFavoriteRating();
	return true;
}

bool ULevelSequenceShotMetaDataLibrary::HasIsNoGood(const ULevelSequence* InLevelSequence)
{
	bool Dummy = false;
	return GetIsNoGood(InLevelSequence, Dummy);
}

bool ULevelSequenceShotMetaDataLibrary::HasIsFlagged(const ULevelSequence* InLevelSequence)
{
	bool Dummy = false;
	return GetIsFlagged(InLevelSequence, Dummy);
}

bool ULevelSequenceShotMetaDataLibrary::HasIsRecorded(const ULevelSequence* InLevelSequence)
{
	bool Dummy = false;
	return GetIsRecorded(InLevelSequence, Dummy);
}

bool ULevelSequenceShotMetaDataLibrary::HasIsSubSequence(const ULevelSequence* InLevelSequence)
{
	bool Dummy = false;
	return GetIsSubSequence(InLevelSequence, Dummy);
}

bool ULevelSequenceShotMetaDataLibrary::HasFavoriteRating(const ULevelSequence* InLevelSequence)
{
	int32 Dummy = 0;
	return GetFavoriteRating(InLevelSequence, Dummy);
}

void ULevelSequenceShotMetaDataLibrary::SetIsNoGood(ULevelSequence* InLevelSequence, bool bInIsNoGood)
{
	if (UMovieSceneShotMetaData* MetaData = UE::Sequencer::TakesMetaDataDetail::FindOrAddMetaData(InLevelSequence))
	{
		MetaData->SetIsNoGood(bInIsNoGood);
	}
}

void ULevelSequenceShotMetaDataLibrary::SetIsFlagged(ULevelSequence* InLevelSequence, bool bInIsFlagged)
{
	if (UMovieSceneShotMetaData* MetaData = UE::Sequencer::TakesMetaDataDetail::FindOrAddMetaData(InLevelSequence))
	{
		MetaData->SetIsFlagged(bInIsFlagged);
	}
}

void ULevelSequenceShotMetaDataLibrary::SetIsRecorded(ULevelSequence* InLevelSequence, bool bInIsRecorded)
{
	if (UMovieSceneShotMetaData* MetaData = UE::Sequencer::TakesMetaDataDetail::FindOrAddMetaData(InLevelSequence))
	{
		MetaData->SetIsRecorded(bInIsRecorded);
	}
}

void ULevelSequenceShotMetaDataLibrary::SetIsSubSequence(ULevelSequence* InLevelSequence, bool bInIsSubSequence)
{
	if (UMovieSceneShotMetaData* MetaData = UE::Sequencer::TakesMetaDataDetail::FindOrAddMetaData(InLevelSequence))
	{
		MetaData->SetIsSubSequence(bInIsSubSequence);
	}
}

void ULevelSequenceShotMetaDataLibrary::SetFavoriteRating(ULevelSequence* InLevelSequence, int32 InFavoriteRating)
{
	if (UMovieSceneShotMetaData* MetaData = UE::Sequencer::TakesMetaDataDetail::FindOrAddMetaData(InLevelSequence))
	{
		MetaData->SetFavoriteRating(InFavoriteRating);
	}
}

void ULevelSequenceShotMetaDataLibrary::ClearIsNoGood(ULevelSequence* InLevelSequence)
{
	if (UMovieSceneShotMetaData* MetaData = UE::Sequencer::TakesMetaDataDetail::FindOrAddMetaData(InLevelSequence))
	{
		MetaData->ClearIsNoGood();
	}
}

void ULevelSequenceShotMetaDataLibrary::ClearIsFlagged(ULevelSequence* InLevelSequence)
{
	if (UMovieSceneShotMetaData* MetaData = UE::Sequencer::TakesMetaDataDetail::FindOrAddMetaData(InLevelSequence))
	{
		MetaData->ClearIsFlagged();
	}
}

void ULevelSequenceShotMetaDataLibrary::ClearIsRecorded(ULevelSequence* InLevelSequence)
{
	if (UMovieSceneShotMetaData* MetaData = UE::Sequencer::TakesMetaDataDetail::FindOrAddMetaData(InLevelSequence))
	{
		MetaData->ClearIsRecorded();
	}
}

void ULevelSequenceShotMetaDataLibrary::ClearIsSubSequence(ULevelSequence* InLevelSequence)
{
	if (UMovieSceneShotMetaData* MetaData = UE::Sequencer::TakesMetaDataDetail::FindOrAddMetaData(InLevelSequence))
	{
		MetaData->ClearIsSubSequence();
	}
}

void ULevelSequenceShotMetaDataLibrary::ClearFavoriteRating(ULevelSequence* InLevelSequence)
{
	if (UMovieSceneShotMetaData* MetaData = UE::Sequencer::TakesMetaDataDetail::FindOrAddMetaData(InLevelSequence))
	{
		MetaData->ClearFavoriteRating();
	}
}

FName ULevelSequenceShotMetaDataLibrary::GetIsNoGoodAssetTag()
{
	return UMovieSceneShotMetaData::AssetRegistryTag_bIsNoGood;
}

FName ULevelSequenceShotMetaDataLibrary::GetIsFlaggedAssetTag()
{
	return UMovieSceneShotMetaData::AssetRegistryTag_bIsFlagged;
}

FName ULevelSequenceShotMetaDataLibrary::GetIsRecordedAssetTag()
{
	return UMovieSceneShotMetaData::AssetRegistryTag_bIsRecorded;
}

FName ULevelSequenceShotMetaDataLibrary::GetIsSubSequenceAssetTag()
{
	return UMovieSceneShotMetaData::AssetRegistryTag_bIsSubSequence;
}

FName ULevelSequenceShotMetaDataLibrary::GetFavoriteRatingAssetTag()
{
	return UMovieSceneShotMetaData::AssetRegistryTag_FavoriteRating;
}

bool ULevelSequenceShotMetaDataLibrary::GetIsNoGoodByAssetData(const FAssetData& InAssetData, bool& bOutNoGood)
{
	return UMovieSceneShotMetaData::GetIsNoGoodByAssetData(InAssetData, bOutNoGood);
}

bool ULevelSequenceShotMetaDataLibrary::GetIsFlaggedByAssetData(const FAssetData& InAssetData, bool& bOutIsFlagged)
{
	return UMovieSceneShotMetaData::GetIsFlaggedByAssetData(InAssetData, bOutIsFlagged);
}

bool ULevelSequenceShotMetaDataLibrary::GetIsRecordedByAssetData(const FAssetData& InAssetData, bool& bOutIsRecorded)
{
	return UMovieSceneShotMetaData::GetIsRecordedByAssetData(InAssetData, bOutIsRecorded);
}

bool ULevelSequenceShotMetaDataLibrary::GetIsSubSequenceByAssetData(const FAssetData& InAssetData, bool& bOutIsSubSequence)
{
	return UMovieSceneShotMetaData::GetIsSubSequenceByAssetData(InAssetData, bOutIsSubSequence);
}

bool ULevelSequenceShotMetaDataLibrary::GetFavoriteRatingByAssetData(const FAssetData& InAssetData, int32& OutFavoriteRating)
{
	return UMovieSceneShotMetaData::GetFavoriteRatingByAssetData(InAssetData, OutFavoriteRating);
}

bool ULevelSequenceShotMetaDataLibrary::HasIsNoGoodByAssetData(const FAssetData& InAssetData)
{
	bool bDummy = false;
	return GetIsNoGoodByAssetData(InAssetData, bDummy);
}

bool ULevelSequenceShotMetaDataLibrary::HasIsFlaggedByAssetData(const FAssetData& InAssetData)
{
	bool bDummy = false;
	return GetIsFlaggedByAssetData(InAssetData, bDummy);
}

bool ULevelSequenceShotMetaDataLibrary::HasIsRecordedByAssetData(const FAssetData& InAssetData)
{
	bool bDummy = false;
	return GetIsRecordedByAssetData(InAssetData, bDummy);
}

bool ULevelSequenceShotMetaDataLibrary::HasIsSubSequenceByAssetData(const FAssetData& InAssetData)
{
	bool bDummy = false;
	return GetIsSubSequenceByAssetData(InAssetData, bDummy);
}

bool ULevelSequenceShotMetaDataLibrary::HasFavoriteRatingByAssetData(const FAssetData& InAssetData)
{
	int32 Dummy = 0;
	return GetFavoriteRatingByAssetData(InAssetData, Dummy);
}
