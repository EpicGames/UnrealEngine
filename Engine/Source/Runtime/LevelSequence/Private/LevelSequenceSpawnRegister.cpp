// Copyright Epic Games, Inc. All Rights Reserved.

#include "LevelSequenceSpawnRegister.h"
#include "Engine/EngineTypes.h"
#include "Engine/NetDriver.h"
#include "MovieScene.h"
#include "MovieSceneSequence.h"
#include "LevelSequenceModule.h"
#include "IMovieSceneObjectSpawner.h"
#include "Modules/ModuleManager.h"
#include "Bindings/MovieSceneSpawnableBinding.h"

FLevelSequenceSpawnRegister::FLevelSequenceSpawnRegister()
{
	FLevelSequenceModule& LevelSequenceModule = FModuleManager::GetModuleChecked<FLevelSequenceModule>("LevelSequence");
	LevelSequenceModule.GenerateObjectSpawners(MovieSceneObjectSpawners);
}

FLevelSequenceSpawnRegister::FLevelSequenceSpawnRegister(const FLevelSequenceSpawnRegister&) = default;
FLevelSequenceSpawnRegister::~FLevelSequenceSpawnRegister() = default;

UObject* FLevelSequenceSpawnRegister::SpawnObject(FMovieSceneSpawnable& Spawnable, FMovieSceneSequenceIDRef TemplateID, TSharedRef<const FSharedPlaybackState> SharedPlaybackState)
{
	for (TSharedRef<IMovieSceneObjectSpawner> MovieSceneObjectSpawner : MovieSceneObjectSpawners)
	{
		if (Spawnable.GetObjectTemplate() != nullptr && Spawnable.GetObjectTemplate()->IsA(MovieSceneObjectSpawner->GetSupportedTemplateType()))
		{
			UObject* SpawnedObject = MovieSceneObjectSpawner->SpawnObject(Spawnable, TemplateID, SharedPlaybackState);
			if (SpawnedObject)
			{
				return SpawnedObject;
			}
		}
	}

	return nullptr;
}

void FLevelSequenceSpawnRegister::DestroySpawnedObject(UObject& Object, UMovieSceneSpawnableBindingBase* CustomSpawnableBinding)
{
	if (CustomSpawnableBinding)
	{
		CustomSpawnableBinding->DestroySpawnedObject(&Object);
	}
	else
	{
		for (TSharedRef<IMovieSceneObjectSpawner> MovieSceneObjectSpawner : MovieSceneObjectSpawners)
		{
			if (Object.IsA(MovieSceneObjectSpawner->GetSupportedTemplateType()))
			{
				UE::Net::FScopedIgnoreStaticActorDestruction ScopedIgnoreDestruction; // net addressable actors normally have a destruction info created for them, we don't want that as these actors were spawned at runtime
				MovieSceneObjectSpawner->DestroySpawnedObject(Object);
				return;
			}
		}

		UE_LOGF(
			LogMovieScene, Error,
			"No valid object spawner found to destroy spawned object '%ls' of type '%ls'.",
			*Object.GetPathName(), *Object.GetClass()->GetName());
	}
}

#if WITH_EDITOR

bool FLevelSequenceSpawnRegister::CanSpawnObject(UClass* InClass) const
{
	for (TSharedRef<IMovieSceneObjectSpawner> MovieSceneObjectSpawner : MovieSceneObjectSpawners)
	{
		if (InClass->IsChildOf(MovieSceneObjectSpawner->GetSupportedTemplateType()))
		{
			return true;
		}
	}
	return false;
}

#endif
