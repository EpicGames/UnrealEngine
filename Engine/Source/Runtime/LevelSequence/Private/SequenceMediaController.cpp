// Copyright Epic Games, Inc. All Rights Reserved.

#include "SequenceMediaController.h"

#include "Engine/World.h"
#include "GameFramework/GameState.h"
#include "Net/UnrealNetwork.h"
#include "TimerManager.h"

#include "IMediaEventSink.h"
#include "MediaComponent.h"
#include "MediaPlayer.h"
#include "MediaPlaylist.h"
#include "LevelSequenceActor.h"
#include "LevelSequencePlayer.h"
#include "LevelSequenceModule.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SequenceMediaController)


ALevelSequenceMediaController::ALevelSequenceMediaController(const FObjectInitializer& Init)
	: Super(Init)
{
	bReplicates = true;
	ServerStartTimeSeconds = -MIN_flt;
	SequencePositionSeconds = 0.0;

	MediaComponent = Init.CreateDefaultSubobject<UMediaComponent>(this, "MediaComponent");
	AddOwnedComponent(MediaComponent);
}

void ALevelSequenceMediaController::GetLifetimeReplicatedProps(TArray<FLifetimeProperty>& OutLifetimeProps) const
{
	Super::GetLifetimeReplicatedProps(OutLifetimeProps);

	DOREPLIFETIME(ALevelSequenceMediaController, ServerStartTimeSeconds);
}

void ALevelSequenceMediaController::OnRep_ServerStartTimeSeconds()
{
	if (ServerStartTimeSeconds != -MIN_flt)
	{
		Client_Play();
	}
}

void ALevelSequenceMediaController::Play()
{
	if (HasAuthority())
	{
		UWorld*         World     = GetWorld();
		AGameStateBase* GameState = World ? World->GetGameState() : nullptr;

		if (GameState)
		{
			ServerStartTimeSeconds = GameState->GetServerWorldTimeSeconds();
		}

		if (GetNetMode() != NM_DedicatedServer)
		{
			OnRep_ServerStartTimeSeconds();
		}
	}
	else
	{
		FFrame::KismetExecutionMessage(*FString::Printf(TEXT("Cannot initiate playback when Sequence does not have authority (%s)"), *GetName()), ELogVerbosity::Warning);
	}
}

void ALevelSequenceMediaController::SynchronizeToServer(float DesyncThresholdSeconds )
{
	if (HasAuthority())
	{
		FFrame::KismetExecutionMessage(*FString::Printf(TEXT("Cannot synchronize when Sequence has authority (%s)"), *GetName()), ELogVerbosity::Warning);
	}
	else
	{
		Client_ConditionallyForceTime(DesyncThresholdSeconds);
	}
}

void ALevelSequenceMediaController::Client_Play()
{
	check(ServerStartTimeSeconds != -MIN_flt);

	if (!Sequence)
	{
		return;
	}

	float CurrentServerTime = 0.f;

	UWorld*         World     = GetWorld();
	AGameStateBase* GameState = World ? World->GetGameState() : nullptr;
	if (GameState)
	{
		CurrentServerTime = GameState->GetServerWorldTimeSeconds() - ServerStartTimeSeconds;
	}

	const float StartTimeSeconds = Sequence->GetSequencePlayer()->GetStartTime().AsSeconds() + CurrentServerTime;

	UE_LOGF(LogLevelSequence, Log, "Initiating playback of sequence '%ls' starting at time %fs", *Sequence->GetName(), StartTimeSeconds);

	Sequence->GetSequencePlayer()->SetPlaybackPosition(FMovieSceneSequencePlaybackParams(StartTimeSeconds, EUpdatePositionMethod::Jump));
	Sequence->GetSequencePlayer()->Play();
}

void ALevelSequenceMediaController::Client_ConditionallyForceTime(float DesyncThresholdSeconds)
{
	UWorld*         World     = GetWorld();
	AGameStateBase* GameState = World ? World->GetGameState() : nullptr;

	if (!GameState || !Sequence)
	{
		return;
	}

	if (!Sequence->GetSequencePlayer()->IsPlaying())
	{
		return;
	}

	const double SequenceOffset = Sequence->GetSequencePlayer()->GetCurrentTime().AsSeconds() - Sequence->GetSequencePlayer()->GetStartTime().AsSeconds();

	const float ExpectedServerTime = ServerStartTimeSeconds + static_cast<float>(SequenceOffset);
	const float CurrentServerTime = GameState->GetServerWorldTimeSeconds();
	
	const float Difference = FMath::Abs(CurrentServerTime - ExpectedServerTime);
	if (Difference > DesyncThresholdSeconds)
	{
		const double NewTimeSeconds = Sequence->GetSequencePlayer()->GetStartTime().AsSeconds() + (CurrentServerTime - ServerStartTimeSeconds);
		Sequence->GetSequencePlayer()->SetPlaybackPosition(FMovieSceneSequencePlaybackParams(NewTimeSeconds, EUpdatePositionMethod::Jump));

		UE_LOGF(LogLevelSequence, Warning, "Forcibly synchronizing sequence '%ls' to time %f to server time (it is out by %fs).", *Sequence->GetName(), NewTimeSeconds, Difference);
	}
}

void ALevelSequenceMediaController::OnStartPlaying(const FQualifiedFrameTime& InStartTime)
{
	SequencePositionSeconds = InStartTime.AsSeconds();
}

void ALevelSequenceMediaController::OnTick(float DeltaSeconds, float InPlayRate)
{
	double StartOffset = 0.0;

	UMediaPlayer* MediaPlayer = MediaComponent->GetMediaPlayer();
	if (MediaPlayer->GetRate() != 0.f || MediaPlayer->IsPlaying())
	{
		SequencePositionSeconds = MediaPlayer->GetTime().GetTotalSeconds();
	}
	else
	{
		SequencePositionSeconds += DeltaSeconds * InPlayRate;
	}
}

void ALevelSequenceMediaController::OnStopPlaying(const FQualifiedFrameTime& InStopTime)
{
}

FFrameTime ALevelSequenceMediaController::OnRequestCurrentTime(const FQualifiedFrameTime& InCurrentTime, float InPlayRate)
{
	return SequencePositionSeconds * InCurrentTime.Rate;
}
