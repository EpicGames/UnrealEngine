// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
#include "UObject/Object.h"
#include "Engine/AssetUserData.h"
#include "AnimSequenceLevelSequenceLink.generated.h"

class ULevelSequence;

/** Link To Level Sequence That may be driving the anim sequence*/
UCLASS(BlueprintType, MinimalAPI)
class UAnimSequenceLevelSequenceLink : public UAssetUserData
{
	GENERATED_UCLASS_BODY()

public:
	virtual bool IsEditorOnly() const override { return true; }

	UPROPERTY(BlueprintReadWrite, Category = Property)
	FGuid SkelTrackGuid;

	UPROPERTY(BlueprintReadWrite, AssetRegistrySearchable, Category = Property)
	FSoftObjectPath PathToLevelSequence;

	LEVELSEQUENCE_API void SetLevelSequence(ULevelSequence* InLevelSequence);
	LEVELSEQUENCE_API ULevelSequence* ResolveLevelSequence();
};
