// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/ObjectMacros.h"
#include "Algo/Transform.h"
#include "Tracks/IMovieSceneTransformOrigin.h"
#include "GameFramework/Actor.h"

#include "DefaultLevelSequenceInstanceData.generated.h"

#define UE_API LEVELSEQUENCE_API

/** Default instance data class that level sequences understand. Implements IMovieSceneTransformOrigin. */
UCLASS(BlueprintType, MinimalAPI)
class UDefaultLevelSequenceInstanceData
	: public UObject
	, public IMovieSceneTransformOrigin
{
public:

	GENERATED_BODY()

	UDefaultLevelSequenceInstanceData(const FObjectInitializer& Init)
		: Super(Init)
	{
		TransformOriginActor = nullptr;
		TransformOrigin = FTransform::Identity;
	}

	virtual FTransform NativeGetTransformOrigin() const override { return TransformOriginActor ? TransformOriginActor->ActorToWorld() : TransformOrigin; }

	/** When set, this actor's world position will be used as the transform origin for all absolute transform sections */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="General")
	TObjectPtr<AActor> TransformOriginActor;

	/** Specifies a transform that offsets all absolute transform sections in this sequence. Scale is ignored. Not applied to Relative or Additive sections. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="General")
	FTransform TransformOrigin;

#if WITH_EDITOR
	DECLARE_MULTICAST_DELEGATE(FTransformOriginChanged);
	FTransformOriginChanged TransformOriginChangedEvent;

	//~ Begin UObject interface
	UE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif // WITH_EDITOR
};

#undef UE_API
