// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "MovieSceneSequence.h"
#include "LevelSequenceBindingReference.h"
#include "LevelSequenceLegacyObjectReference.h"
#include "Templates/SubclassOf.h"
#include "Interfaces/Interface_AssetUserData.h"
#include "LevelSequence.generated.h"

class UBlueprint;
class UMovieScene;
class UBlueprintGeneratedClass;
class UAssetUserData;

/**
 * Movie scene animation for Actors.
 */
UCLASS(BlueprintType, MinimalAPI)
class ULevelSequence
	: public UMovieSceneSequence, public IInterface_AssetUserData
{
	GENERATED_UCLASS_BODY()

public:

	/** Pointer to the movie scene that controls this animation. */
	UPROPERTY()
	TObjectPtr<UMovieScene> MovieScene;

public:

	/** Initialize this level sequence. */
	LEVELSEQUENCE_API virtual void Initialize();

public:

	// UMovieSceneSequence interface
	LEVELSEQUENCE_API virtual void BindPossessableObject(const FGuid& ObjectId, UObject& PossessedObject, UObject* Context) override;
	LEVELSEQUENCE_API virtual bool CanPossessObject(UObject& Object, UObject* InPlaybackContext) const override;

	UE_DEPRECATED(5.5, "Use the version that takes a SharedPlaybackState")
	LEVELSEQUENCE_API virtual FGuid FindBindingFromObject(UObject* InObject, UObject* Context) const override;
	LEVELSEQUENCE_API virtual FGuid FindBindingFromObject(UObject* InObject, TSharedRef<const FSharedPlaybackState> SharedPlaybackState) const override;

	LEVELSEQUENCE_API virtual void GatherExpiredObjects(const FMovieSceneObjectCache& InObjectCache, TArray<FGuid>& OutInvalidIDs) const override;
	LEVELSEQUENCE_API virtual UMovieScene* GetMovieScene() const override;
	LEVELSEQUENCE_API virtual UObject* GetParentObject(UObject* Object) const override;
	LEVELSEQUENCE_API virtual void UnbindPossessableObjects(const FGuid& ObjectId) override;
	LEVELSEQUENCE_API virtual void UnbindObjects(const FGuid& ObjectId, const TArray<UObject*>& InObjects, UObject* InContext) override;
	LEVELSEQUENCE_API virtual void UnbindInvalidObjects(const FGuid& ObjectId, UObject* InContext) override;
	LEVELSEQUENCE_API virtual bool AllowsSpawnableObjects() const override;
	LEVELSEQUENCE_API virtual bool AllowsCustomBindings() const override;
	LEVELSEQUENCE_API virtual bool CanRebindPossessable(const FMovieScenePossessable& InPossessable) const override;
	LEVELSEQUENCE_API virtual UObject* MakeSpawnableTemplateFromInstance(UObject& InSourceObject, FName ObjectName) override;
	LEVELSEQUENCE_API virtual bool CanAnimateObject(UObject& InObject) const override;
	LEVELSEQUENCE_API virtual UObject* CreateDirectorInstance(TSharedRef<const FSharedPlaybackState> SharedPlaybackState, FMovieSceneSequenceID SequenceID) override;
	LEVELSEQUENCE_API virtual const FMovieSceneBindingReferences* GetBindingReferences() const override;
	LEVELSEQUENCE_API virtual void PostLoad() override;
#if WITH_EDITORONLY_DATA
	static LEVELSEQUENCE_API void DeclareConstructClasses(TArray<FTopLevelAssetPath>& OutConstructClasses, const UClass* SpecificSubclass);
#endif

	LEVELSEQUENCE_API virtual void PostInitProperties() override;
	LEVELSEQUENCE_API virtual bool Rename(const TCHAR* NewName = nullptr, UObject* NewOuter = nullptr, ERenameFlags Flags = REN_None) override;


	//~ Begin IInterface_AssetUserData Interface
	LEVELSEQUENCE_API virtual void AddAssetUserData(UAssetUserData* InUserData) override;
	LEVELSEQUENCE_API virtual void RemoveUserDataOfClass(TSubclassOf<UAssetUserData> InUserDataClass) override;
	LEVELSEQUENCE_API virtual UAssetUserData* GetAssetUserDataOfClass(TSubclassOf<UAssetUserData> InUserDataClass) override;
	LEVELSEQUENCE_API virtual const TArray<UAssetUserData*>* GetAssetUserDataArray() const override;
	//~ End IInterface_AssetUserData Interface


#if WITH_EDITOR
	LEVELSEQUENCE_API virtual ETrackSupport IsTrackSupportedImpl(TSubclassOf<class UMovieSceneTrack> InTrackClass) const override;
	LEVELSEQUENCE_API virtual bool IsFilterSupportedImpl(const FString& InFilterName) const override;
	LEVELSEQUENCE_API virtual void GetAssetRegistryTagMetadata(TMap<FName, FAssetRegistryTagMetadata>& OutMetadata) const override;
	LEVELSEQUENCE_API virtual void GetAssetRegistryTags(FAssetRegistryTagsContext Context) const override;
	UE_DEPRECATED(5.4, "Implement the version that takes FAssetRegistryTagsContext instead.")
	LEVELSEQUENCE_API virtual void GetAssetRegistryTags(TArray<FAssetRegistryTag>& OutTags) const override;
protected:
	LEVELSEQUENCE_API virtual void ThreadedPostLoadAssetRegistryTagsOverride(FPostLoadAssetRegistryTagsContext& Context) const override;
public:

	DECLARE_DELEGATE_RetVal_OneParam(void, FPostDuplicateEvent, ULevelSequence*);
	static LEVELSEQUENCE_API FPostDuplicateEvent PostDuplicateEvent;

	DECLARE_MULTICAST_DELEGATE_OneParam(FFixupDynamicBindingsEvent, ULevelSequence*);
	static LEVELSEQUENCE_API FFixupDynamicBindingsEvent FixupDynamicBindingsEvent;


#endif

	LEVELSEQUENCE_API virtual void PostDuplicate(bool bDuplicateForPIE) override;

	using UMovieSceneSequence::LocateBoundObjects;

	UE_DEPRECATED(5.4, "Use the base class LocateBoundObjects()")
	LEVELSEQUENCE_API void LocateBoundObjects(const FGuid& ObjectId, UObject* Context, const FLevelSequenceBindingReference::FResolveBindingParams& InResolveBindingParams, TArray<UObject*, TInlineAllocator<1>>& OutObjects) const;

	
#if WITH_EDITOR

	LEVELSEQUENCE_API virtual void IterateDynamicBindings(TFunction<void(const FGuid&, FMovieSceneDynamicBinding&)> InCallback) override;

public:

	/**
	 * Assign a new director blueprint to this level sequence. The specified blueprint *must* be contained within this object.?	 */
	LEVELSEQUENCE_API void SetDirectorBlueprint(UBlueprint* NewDirectorBlueprint);

	/**
	 * Retrieve the currently assigned director blueprint for this level sequence
	 */
	LEVELSEQUENCE_API UBlueprint* GetDirectorBlueprint() const;

	LEVELSEQUENCE_API FString GetDirectorBlueprintName() const;

protected:

	LEVELSEQUENCE_API virtual FGuid CreatePossessable(UObject* ObjectToPossess) override;
	LEVELSEQUENCE_API virtual FGuid CreateSpawnable(UObject* ObjectToSpawn) override;

	LEVELSEQUENCE_API FGuid FindOrAddBinding(UObject* ObjectToPossess);

	/**
	 * Invoked when this level sequence's director blueprint has been recompiled
	 */
	LEVELSEQUENCE_API void OnDirectorRecompiled(UBlueprint*);

private:
#if WITH_EDITOR
	// Used to convert any old spawnables in this movie scene to use the custom binding system
	bool ConvertOldSpawnables();
	// Used to convert possessables using deprecated 'DynamicBinding' to a custom director blueprint binding
	void ConvertDynamicBindingPossessable(FMovieScenePossessable& Possessable);
	void OnEndLoadPackage(const FEndLoadPackageContext& Context);
#endif

#endif // WITH_EDITOR

protected:

	/** References to bound objects. */
	UPROPERTY()
	FUpgradedLevelSequenceBindingReferences BindingReferences;

#if WITH_EDITORONLY_DATA

	/** Legacy object references */
	UPROPERTY()
	FLevelSequenceObjectReferenceMap ObjectReferences_DEPRECATED;

	/** A pointer to the director blueprint that generates this sequence's DirectorClass. */
	UPROPERTY()
	TObjectPtr<UBlueprint> DirectorBlueprint;

#endif

	/**
	 * The class that is used to spawn this level sequence's director instance.
	 * Director instances are allocated on-demand one per sequence during evaluation and are used by event tracks for triggering events.
	 */
	UPROPERTY()
	TObjectPtr<UClass> DirectorClass;

public:
	/**
	* Find meta-data of a particular type for this level sequence instance.
	* @param InClass - Class that you wish to find the metadata object for.
	* @return An instance of this class if it already exists as metadata on this Level Sequence, otherwise null.
	*/
	UFUNCTION(BlueprintPure, Category = "Level Sequence", meta=(DevelopmentOnly))
	UObject* FindMetaDataByClass(TSubclassOf<UObject> InClass) const
	{
#if WITH_EDITORONLY_DATA
		auto const* Found = MetaDataObjects.FindByPredicate([InClass](UObject* In) { return In && In->GetClass() == InClass; });
		return Found ? CastChecked<UObject>(*Found) : nullptr;
#else
		return nullptr;
#endif
	}

	/**
	* Find meta-data of a particular type for this level sequence instance, adding it if it doesn't already exist.
	* @param InClass - Class that you wish to find or create the metadata object for.
	* @return An instance of this class as metadata on this Level Sequence.
	*/
	UFUNCTION(BlueprintCallable, Category = "Level Sequence", meta=(DevelopmentOnly))
	UObject* FindOrAddMetaDataByClass(TSubclassOf<UObject> InClass)
	{
#if WITH_EDITORONLY_DATA
		UObject* Found = FindMetaDataByClass(InClass);
		if (!Found)
		{
			Found = NewObject<UObject>(this, InClass);
			MetaDataObjects.Add(Found);
		}
		return Found;
#else
		return nullptr;
#endif
	}

	/**
	* Copy the specified meta data into this level sequence, overwriting any existing meta-data of the same type
	* Meta-data may implement the ILevelSequenceMetaData interface in order to hook into default ULevelSequence functionality.
	* @param InMetaData - Existing Metadata Object that you wish to copy into this Level Sequence.
	* @return The newly copied instance of the Metadata that now exists on this sequence.
	*/
	UFUNCTION(BlueprintCallable, Category = "Level Sequence", meta=(DevelopmentOnly))
	UObject* CopyMetaData(UObject* InMetaData)
	{
#if WITH_EDITORONLY_DATA
		if (!InMetaData)
		{
			return nullptr;
		}

		RemoveMetaDataByClass(InMetaData->GetClass());

		UObject* NewMetaData = DuplicateObject(InMetaData, this);
		MetaDataObjects.Add(NewMetaData);

		return NewMetaData;
#else
		return nullptr;
#endif	
	}

	/**
	* Remove meta-data of a particular type for this level sequence instance, if it exists
	* @param InClass - The class type that you wish to remove the metadata for
	*/
	UFUNCTION(BlueprintCallable, Category = "Level Sequence", meta=(DevelopmentOnly))
	void RemoveMetaDataByClass(TSubclassOf<UObject> InClass)
	{
#if WITH_EDITORONLY_DATA
		MetaDataObjects.RemoveAll([InClass](UObject* In) { return In && In->GetClass() == InClass; });
#endif
	}

#if WITH_EDITORONLY_DATA

public:

	/**
	 * Find meta-data of a particular type for this level sequence instance
	 */
	template<typename MetaDataType>
	MetaDataType* FindMetaData() const
	{
		UClass* PredicateClass = MetaDataType::StaticClass();
		auto const* Found = MetaDataObjects.FindByPredicate([PredicateClass](UObject* In){ return In && In->GetClass() == PredicateClass; });
		return Found ? CastChecked<MetaDataType>(*Found) : nullptr;
	}

	/**
	 * Find meta-data of a particular type for this level sequence instance, adding one if it was not found.
	 * Meta-data may implement the ILevelSequenceMetaData interface in order to hook into default ULevelSequence functionality.
	 */
	template<typename MetaDataType>
	MetaDataType* FindOrAddMetaData()
	{
		MetaDataType* Found = FindMetaData<MetaDataType>();
		if (!Found)
		{
			Found = NewObject<MetaDataType>(this);
			MetaDataObjects.Add(Found);
		}
		return Found;
	}

	/**
	 * Copy the specified meta data into this level sequence, overwriting any existing meta-data of the same type
	 * Meta-data may implement the ILevelSequenceMetaData interface in order to hook into default ULevelSequence functionality.
	 */
	template<typename MetaDataType>
	MetaDataType* CopyMetaData(MetaDataType* InMetaData)
	{
		RemoveMetaData<MetaDataType>();

		MetaDataType* NewMetaData = DuplicateObject(InMetaData, this);
		MetaDataObjects.Add(NewMetaData);

		return NewMetaData;
	}

	/**
	 * Remove meta-data of a particular type for this level sequence instance, if it exists
	 */
	template<typename MetaDataType>
	void RemoveMetaData()
	{
		UClass* PredicateClass = MetaDataType::StaticClass();
		MetaDataObjects.RemoveAll([PredicateClass](UObject* In){ return In && In->GetClass() == PredicateClass; });
	}

private:

	/** Array of meta-data objects associated with this level sequence. Each pointer may implement the ILevelSequenceMetaData interface in order to hook into default ULevelSequence functionality. */
	UPROPERTY()
	TArray<TObjectPtr<UObject>> MetaDataObjects;

#endif

protected:
	/** Array of user data stored with the asset */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Instanced, Category = Animation)
	TArray<TObjectPtr<UAssetUserData>> AssetUserData;
};
