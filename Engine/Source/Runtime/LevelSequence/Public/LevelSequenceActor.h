// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "UObject/SoftObjectPath.h"
#include "GameFramework/Actor.h"
#include "IMovieScenePlaybackClient.h"
#include "MovieSceneSequencePlaybackSettings.h"
#include "MovieSceneBindingOwnerInterface.h"
#include "MovieSceneBindingOverrides.h"
#include "LevelSequenceCameraSettings.h"
#include "WorldPartition/IWorldPartitionObjectResolver.h"

#include "LevelSequenceActor.generated.h"

class ULevelSequenceBurnIn;
class ULevelSequencePlayer;
class UMovieSceneSequenceTickManager;

UCLASS(Blueprintable, DefaultToInstanced, MinimalAPI)
class ULevelSequenceBurnInInitSettings : public UObject
{
	GENERATED_BODY()
};

UCLASS(config=EditorPerProjectUserSettings, PerObjectConfig, DefaultToInstanced, BlueprintType, MinimalAPI)
class ULevelSequenceBurnInOptions : public UObject
{
public:

	GENERATED_BODY()
	LEVELSEQUENCE_API ULevelSequenceBurnInOptions(const FObjectInitializer& Init);

	/** Loads the specified class path and initializes an instance, then stores it in Settings. */
	UFUNCTION(BlueprintCallable, Category = "General")
	LEVELSEQUENCE_API void SetBurnIn(FSoftClassPath InBurnInClass);

	/** Ensure the settings object is up-to-date */
	LEVELSEQUENCE_API void ResetSettings();

public:

	UPROPERTY(config, EditAnywhere, BlueprintReadWrite, Category="General")
	bool bUseBurnIn;

	UPROPERTY(config, EditAnywhere, BlueprintReadWrite, Category="General", meta=(EditCondition=bUseBurnIn, MetaClass="/Script/LevelSequence.LevelSequenceBurnIn"))
	FSoftClassPath BurnInClass;

	UPROPERTY(Instanced, EditAnywhere, BlueprintReadWrite, Category="General", meta=(EditCondition=bUseBurnIn))
	TObjectPtr<ULevelSequenceBurnInInitSettings> Settings;

protected:

#if WITH_EDITOR
	LEVELSEQUENCE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif //WITH_EDITOR
};

/**
 * Actor responsible for controlling a specific level sequence in the world.
 */
UCLASS(hideCategories=(Rendering, Physics, HLOD, Activation, Input), MinimalAPI)
class ALevelSequenceActor
	: public AActor
	, public IMovieScenePlaybackClient
	, public IMovieSceneBindingOwnerInterface
	, public IWorldPartitionObjectResolver
{
public:

	DECLARE_DYNAMIC_DELEGATE(FOnLevelSequenceLoaded);

	GENERATED_BODY()

	/** Create and initialize a new instance. */
	LEVELSEQUENCE_API ALevelSequenceActor(const FObjectInitializer& Init);

public:

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category="Playback", meta=(ShowOnlyInnerProperties, ExposeOnSpawn))
	FMovieSceneSequencePlaybackSettings PlaybackSettings;

	UE_DEPRECATED(5.4, "Direct access to SequencePlayer will be removed. Please use GetSequencePlayer()")
	UPROPERTY(Instanced, transient, replicated, BlueprintReadOnly, BlueprintGetter = GetSequencePlayer, Category = "Playback", meta = (ExposeFunctionCategories = "Sequencer|Player"))
	TObjectPtr<ULevelSequencePlayer> SequencePlayer;

	UPROPERTY(EditAnywhere, replicated, BlueprintReadOnly, Category="General", meta=(AllowedClasses="/Script/LevelSequence.LevelSequence", ExposeOnSpawn))
	TObjectPtr<ULevelSequence> LevelSequenceAsset;

#if WITH_EDITORONLY_DATA
	UPROPERTY()
	FSoftObjectPath LevelSequence_DEPRECATED;
#endif

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Cameras", meta=(ShowOnlyInnerProperties, ExposeOnSpawn))
	FLevelSequenceCameraSettings CameraSettings;

	UPROPERTY(Instanced, BlueprintReadOnly, Category="General")
	TObjectPtr<ULevelSequenceBurnInOptions> BurnInOptions;

	/** Mapping of actors to override the sequence bindings with */
	UPROPERTY(Instanced, BlueprintReadOnly, Category="General")
	TObjectPtr<UMovieSceneBindingOverrides> BindingOverrides;

	UPROPERTY()
	uint8 bAutoPlay_DEPRECATED : 1;

	/** Enable specification of dynamic instance data to be supplied to the sequence during playback */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="General")
	uint8 bOverrideInstanceData : 1;

	/** If true, playback of this level sequence on the server will be synchronized across other clients */
	UPROPERTY(EditAnywhere, DisplayName="Replicate Playback", BlueprintReadWrite, BlueprintSetter=SetReplicatePlayback, Category=Replication, meta=(ExposeOnSpawn))
	uint8 bReplicatePlayback:1;

	/** Instance data that can be used to dynamically control sequence evaluation at runtime */
	UPROPERTY(Instanced, BlueprintReadWrite, Category="General")
	TObjectPtr<UObject> DefaultInstanceData;

public:

	/**
	 * Get the level sequence being played by this actor.
	 *
	 * @return Level sequence, or nullptr if not assigned or if it cannot be loaded.
	 * @see SetSequence
	 */
	UFUNCTION(BlueprintCallable, Category="Sequencer|Player")
	LEVELSEQUENCE_API ULevelSequence* GetSequence() const;

	/**
	 * Set the level sequence being played by this actor.
	 *
	 * @param InSequence The sequence object to set.
	 * @see GetSequence
	 */
	UFUNCTION(BlueprintCallable, Category="Sequencer|Player")
	LEVELSEQUENCE_API void SetSequence(ULevelSequence* InSequence);

	/**
	 * Set whether or not to replicate playback for this actor
	 */
	UFUNCTION(BlueprintSetter)
	LEVELSEQUENCE_API void SetReplicatePlayback(bool ReplicatePlayback);

	/**
	 * Access this actor's sequence player, or None if it is not yet initialized
	 */
	UFUNCTION(BlueprintGetter)
	LEVELSEQUENCE_API ULevelSequencePlayer* GetSequencePlayer() const;

	/* Hide burnin */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Player")
	LEVELSEQUENCE_API void HideBurnin();

	/* Show burnin */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Player")
	LEVELSEQUENCE_API void ShowBurnin();

	/** Refresh this actor's burn in */
	LEVELSEQUENCE_API void RefreshBurnIn();

public:

	/**
	 * Overrides the specified binding with the specified actors, optionally still allowing the bindings defined in the Level Sequence asset
	 *
	 * @param Binding Binding to modify
	 * @param Actors Actors to bind
	 * @param bAllowBindingsFromAsset If false the new bindings being supplied here will replace the bindings set in the level sequence asset, meaning the original object animated by 
	 *								  Sequencer will no longer be animated. Bindings set to spawnables will not spawn if false. If true, new bindings will be in addition to ones set
	 *								  set in Sequencer UI. This function will not modify the original asset.
	 */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Player|Bindings")
	LEVELSEQUENCE_API void SetBinding(FMovieSceneObjectBindingID Binding, const TArray<AActor*>& Actors, bool bAllowBindingsFromAsset = false);

	/**
	 * Assigns an set of actors to all the bindings tagged with the specified name in this sequence. Object Bindings can be tagged within the sequence UI by RMB -> Tags... on the object binding in the tree.
	 *
	 * @param BindingTag   The unique tag name to lookup bindings with
	 * @param Actors       The actors to assign to all the tagged bindings
	 * @param bAllowBindingsFromAsset If false the new bindings being supplied here will replace the bindings set in the level sequence asset, meaning the original object animated by 
	 *								  Sequencer will no longer be animated. Bindings set to spawnables will not spawn if false. If true, new bindings will be in addition to ones set
	 *								  set in Sequencer UI. This function will not modify the original asset.
	 */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Player|Bindings")
	LEVELSEQUENCE_API void SetBindingByTag(FName BindingTag, const TArray<AActor*>& Actors, bool bAllowBindingsFromAsset = false);

	/**
	 * Adds the specified actor to the overridden bindings for the specified binding ID, optionally still allowing the bindings defined in the Level Sequence asset
	 *
	 * @param Binding Binding to modify
	 * @param Actor Actor to bind
	 * @param bAllowBindingsFromAsset If false the new bindings being supplied here will replace the bindings set in the level sequence asset, meaning the original object animated by 
	 *								  Sequencer will no longer be animated. Bindings set to spawnables will not spawn if false. If true, new bindings will be in addition to ones set
	 *								  set in Sequencer UI. This function will not modify the original asset.
	 */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Player|Bindings")
	LEVELSEQUENCE_API void AddBinding(FMovieSceneObjectBindingID Binding, AActor* Actor, bool bAllowBindingsFromAsset = false);

	/**
	 * Binds an actor to all the bindings tagged with the specified name in this sequence. Does not remove any exising bindings that have been set up through this API. Object Bindings can be tagged within the sequence UI by RMB -> Tags... on the object binding in the tree.
	 *
	 * @param BindingTag   The unique tag name to lookup bindings with
	 * @param Actor        The actor to assign to all the tagged bindings
	 * @param bAllowBindingsFromAsset If false the new bindings being supplied here will replace the bindings set in the level sequence asset, meaning the original object animated by 
	 *								  Sequencer will no longer be animated. Bindings set to spawnables will not spawn if false. If true, new bindings will be in addition to ones set
	 *								  set in Sequencer UI. This function will not modify the original asset.
	 */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Player|Bindings")
	LEVELSEQUENCE_API void AddBindingByTag(FName BindingTag, AActor* Actor, bool bAllowBindingsFromAsset = false);

	/**
	 * Removes the specified actor from the specified binding's actor array
	 */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Player|Bindings")
	LEVELSEQUENCE_API void RemoveBinding(FMovieSceneObjectBindingID Binding, AActor* Actor);

	/**
	 * Removes the specified actor from the specified binding's actor array
	 */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Player|Bindings")
	LEVELSEQUENCE_API void RemoveBindingByTag(FName Tag, AActor* Actor);

	/**
	 * Resets the specified binding back to the defaults defined by the Level Sequence asset
	 */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Player|Bindings")
	LEVELSEQUENCE_API void ResetBinding(FMovieSceneObjectBindingID Binding);

	/**
	 * Resets all overridden bindings back to the defaults defined by the Level Sequence asset
	 */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Player|Bindings")
	LEVELSEQUENCE_API void ResetBindings();

	/**
	 * Retrieve the first object binding that has been tagged with the specified name
	 */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Player|Bindings", DisplayName="Find Binding by Tag")
	LEVELSEQUENCE_API FMovieSceneObjectBindingID FindNamedBinding(FName Tag) const;

	/**
	 * Retrieve all the bindings that have been tagged with the specified name
	 *
	 * @param Tag  The unique tag name to lookup bindings with. Object Bindings can be tagged within the sequence UI by RMB -> Tags... on the object binding in the tree.
	 * @return An array containing all the bindings that are tagged with this name, potentially empty.
	 */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Player|Bindings", DisplayName="Find Bindings by Tag")
	LEVELSEQUENCE_API const TArray<FMovieSceneObjectBindingID>& FindNamedBindings(FName Tag) const;

protected:

	//~ Begin IMovieScenePlaybackClient interface
	LEVELSEQUENCE_API virtual bool RetrieveBindingOverrides(const FGuid& InBindingId, FMovieSceneSequenceID InSequenceID, TArray<UObject*, TInlineAllocator<1>>& OutObjects) const override;
	LEVELSEQUENCE_API virtual UObject* GetInstanceData() const override;
	LEVELSEQUENCE_API virtual TOptional<EAspectRatioAxisConstraint> GetAspectRatioAxisConstraint() const override;
	LEVELSEQUENCE_API virtual bool GetIsReplicatedPlayback() const override;
#if WITH_EDITOR
	LEVELSEQUENCE_API virtual bool CanPreview(const UMovieSceneSequence& InSequence) const override;
#endif
	//~ End IMovieScenePlaybackClient interface

	//~ Begin UObject interface
	LEVELSEQUENCE_API virtual void PostInitProperties() override;
	LEVELSEQUENCE_API virtual void PostLoad() override;

#if WITH_EDITOR
	LEVELSEQUENCE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;

	void TransformOriginChanged();
#endif //WITH_EDITOR

public:
#if WITH_EDITORONLY_DATA
	static LEVELSEQUENCE_API void DeclareConstructClasses(TArray<FTopLevelAssetPath>& OutConstructClasses, const UClass* SpecificSubclass);
#endif
protected:

	//~ End UObject interface

	//~ Begin AActor interface
	LEVELSEQUENCE_API virtual void PreInitializeComponents() override;
	LEVELSEQUENCE_API virtual void PostInitializeComponents() override;
	LEVELSEQUENCE_API virtual void BeginPlay() override;
	LEVELSEQUENCE_API virtual void EndPlay(const EEndPlayReason::Type EndPlayReason) override;
	LEVELSEQUENCE_API virtual void RewindForReplay() override;
	LEVELSEQUENCE_API virtual void PostNetReceive() override;
#if WITH_EDITOR
	virtual bool CanChangeIsSpatiallyLoadedFlag() const override { return false; }
#endif
	//~ End AActor interface

public:

	const FWorldPartitionResolveData& GetWorldPartitionResolveData() const
	{
		return WorldPartitionResolveData;
	}

#if WITH_EDITOR
	virtual void SetWorldPartitionResolveData(const FWorldPartitionResolveData& InWorldPartitionResolveData)
	{
		WorldPartitionResolveData = InWorldPartitionResolveData;
	}

	LEVELSEQUENCE_API virtual bool GetReferencedContentObjects(TArray<UObject*>& Objects) const override;
#endif //WITH_EDITOR

	/** Initialize the player object by loading the asset, using async loading when necessary */
	LEVELSEQUENCE_API void InitializePlayer();

	/** Initialize the player object with the specified asset */
	LEVELSEQUENCE_API void InitializePlayerWithSequence(ULevelSequence* LevelSequenceAsset);
	LEVELSEQUENCE_API void OnSequenceLoaded(const FName& PackageName, UPackage* Package, EAsyncLoadingResult::Type Result);

#if WITH_EDITOR
	LEVELSEQUENCE_API virtual TSharedPtr<FStructOnScope> GetObjectPickerProxy(TSharedPtr<IPropertyHandle> PropertyHandle) override;
	LEVELSEQUENCE_API virtual void UpdateObjectFromProxy(FStructOnScope& Proxy, IPropertyHandle& ObjectPropertyHandle) override;
	LEVELSEQUENCE_API virtual UMovieSceneSequence* RetrieveOwnedSequence() const override;
#endif

private:
	/** Burn-in widget */
	UPROPERTY()
	TObjectPtr<ULevelSequenceBurnIn> BurnInInstance;

	UPROPERTY()
	bool bShowBurnin;

	UPROPERTY()
	FWorldPartitionResolveData WorldPartitionResolveData;
};

USTRUCT()
struct FBoundActorProxy
{
	GENERATED_BODY()

#if WITH_EDITORONLY_DATA

	/** Specifies the actor to override the binding with */
	UPROPERTY(EditInstanceOnly, AdvancedDisplay, Category="General")
	TObjectPtr<AActor> BoundActor = nullptr;

	LEVELSEQUENCE_API void Initialize(TSharedPtr<IPropertyHandle> InPropertyHandle);

	LEVELSEQUENCE_API void OnReflectedPropertyChanged();

	TSharedPtr<IPropertyHandle> ReflectedProperty;

#endif
};

/**
 * A level sequence actor that is set to always be relevant for networking purposes
 */
UCLASS(MinimalAPI)
class AReplicatedLevelSequenceActor
	: public ALevelSequenceActor
{
	GENERATED_BODY()

public:
	/** Create and initialize a new instance. */
	LEVELSEQUENCE_API AReplicatedLevelSequenceActor(const FObjectInitializer& Init);
};
