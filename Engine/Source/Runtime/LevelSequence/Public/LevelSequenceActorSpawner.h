// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IMovieSceneObjectSpawner.h"

class FLevelSequenceActorSpawner : public IMovieSceneObjectSpawner
{
public:

	static LEVELSEQUENCE_API TSharedRef<IMovieSceneObjectSpawner> CreateObjectSpawner();

	// IMovieSceneObjectSpawner interface
	LEVELSEQUENCE_API virtual UClass* GetSupportedTemplateType() const override;
	LEVELSEQUENCE_API virtual UObject* SpawnObject(FMovieSceneSpawnable& Spawnable, FMovieSceneSequenceIDRef TemplateID, TSharedRef<const UE::MovieScene::FSharedPlaybackState> SharedPlaybackState) override;
	LEVELSEQUENCE_API virtual void DestroySpawnedObject(UObject& Object) override;
};
