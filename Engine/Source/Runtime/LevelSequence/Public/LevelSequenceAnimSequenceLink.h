// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Containers/Array.h"
#include "Engine/AssetUserData.h"
#include "Misc/Guid.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/SoftObjectPath.h"
#include "UObject/UObjectGlobals.h"
#include "Animation/AnimTypes.h"
#include "Curves/RealCurve.h"
#include "LevelSequenceAnimSequenceLink.generated.h"

class UAnimSequence;
class UObject;

/** Link To Anim Sequence that we are linked too.*/
USTRUCT(BlueprintType)
struct FLevelSequenceAnimSequenceLinkItem
{
	GENERATED_BODY()

public:
	UPROPERTY(BlueprintReadWrite, Category = Property)
	FGuid SkelTrackGuid;

	UPROPERTY(BlueprintReadWrite, Category = Property)
	FSoftObjectPath PathToAnimSequence;

	//if true it will be autobaked
	UPROPERTY(BlueprintReadWrite, Category = Property)
	bool bAutoBake = false;

	//From Editor Only UAnimSeqExportOption we cache this since we can re-import dynamically
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Property)
	bool bExportTransforms = true;
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Property)
	bool bExportMorphTargets = true;
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Property)
	bool bExportAttributeCurves = true;
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Property)
	bool bExportMaterialCurves = true;
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Property)
	bool bSkipCurvesWithZeroValue = false;
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Property)
	bool bSetRetargetSourceAsset = true;
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Property);
	EAnimInterpolationType Interpolation = EAnimInterpolationType::Linear;
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Property);
	TEnumAsByte<ERichCurveInterpMode> CurveInterpolation = ERichCurveInterpMode::RCIM_Linear;
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Property)
	bool bRecordInWorldSpace = false;
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Property)
	bool bEvaluateAllSkeletalMeshComponents = true;

	/** Include only the animation bones/curves that match this list */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Property);
	TArray<FString> IncludeAnimationNames;
	/** Exclude all animation bones/curves that match this list */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Property);
	TArray<FString> ExcludeAnimationNames;
	/** If enabled, curves in ExcludeAnimationNames that already exist in the target sequence will be removed at the start of recording */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Property)
	bool bRemoveExcludedCurves = false;
	/** Number of Display Rate frames to evaluate before doing the export. It will evaluate after any Delay. This will use frames before the start frame. Use it if there is some post anim BP effects you want to run before export start time.*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Property);
	FFrameNumber WarmUpFrames = 0;
	/** Number of Display Rate frames to delay at the same frame before doing the export. It will evalaute first, then any warm up, then the export. Use it if there is some post anim BP effects you want to ran repeatedly at the start.*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Property);
	FFrameNumber DelayBeforeStart = 0;
	/** Whether or not to use custom time range */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Property)
	bool bUseCustomTimeRange = false;
	/** Custom start frame in display rate*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, meta = (EditCondition = bUseCustomTimeRange), Category = Property)
	FFrameNumber CustomStartFrame = 0;
	/** Custom end frame in display rate */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, meta = (EditCondition = bUseCustomTimeRange), Category = Property)
	FFrameNumber CustomEndFrame = 120;
	/** Custom display rate, should be set from the movie scene/sequencer display rate */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, meta = (EditCondition = bUseCustomTimeRange), Category = Property)
	FFrameRate CustomDisplayRate = FFrameRate(30, 1);

	/** Whether or not to use custom frame rate or Sequencer display rate */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Property)
	bool bUseCustomFrameRate = false;

	/** Custom frame rate that the anim sequence may have been recorded at */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, meta = (EditCondition = bUseCustomFrameRate), Category = Property)
	FFrameRate CustomFrameRate = FFrameRate(30, 1);

	LEVELSEQUENCE_API void SetAnimSequence(UAnimSequence* InAnimSequence);
	LEVELSEQUENCE_API UAnimSequence* ResolveAnimSequence();

	bool IsEqual(FGuid InSkelTrackGuid, bool bInUseCustomTimeRange = false,
		FFrameNumber InCustomStartFrame = 0, FFrameNumber InCustomEndFrame = 120, FFrameRate InCustomDisplayRate = FFrameRate(30,1),
		bool bInUseCustomFrameRate = false, FFrameRate InCustomFrameRate = FFrameRate(30, 1)
		)
	{
		if (InSkelTrackGuid == SkelTrackGuid)
		{
			if (bUseCustomTimeRange == bInUseCustomTimeRange)
			{
				if (bUseCustomTimeRange == false || (InCustomStartFrame == CustomStartFrame &&
					InCustomEndFrame == CustomEndFrame && InCustomDisplayRate == CustomDisplayRate))
				{
					if (bUseCustomFrameRate == bInUseCustomFrameRate)
					{
						if (bUseCustomFrameRate == false || (InCustomFrameRate == CustomFrameRate))
						{
							return true;
						}
					}
				}
			}
		}
		return false;
	}
};

/** Link To Set of Anim Sequences that we may belinked to.*/
UCLASS(BlueprintType, MinimalAPI)
class ULevelSequenceAnimSequenceLink : public UAssetUserData
{
	GENERATED_UCLASS_BODY()

public:
	virtual bool IsEditorOnly() const override { return true; }

	UPROPERTY(BlueprintReadWrite, Category = Links)
	TArray< FLevelSequenceAnimSequenceLinkItem> AnimSequenceLinks;
};
