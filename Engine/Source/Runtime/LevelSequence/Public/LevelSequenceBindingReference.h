// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/SoftObjectPath.h"
#include "Misc/Guid.h"
#include "Engine/Engine.h"
#include "Misc/Paths.h"
#include "MovieSceneBindingReferences.h"
#include "LevelSequenceBindingReference.generated.h"

class UWorld;
struct FWorldPartitionResolveData;

/**
 * An external reference to an level sequence object, resolvable through an arbitrary context.
 * 
 * Bindings consist of an optional package name, and the path to the object within that package.
 * Where package name is empty, the reference is a relative path from a specific outer (the context).
 * Currently, the package name should only ever be empty for component references, which must remain relative bindings to work correctly with spawnables and reinstanced actors.
 */
USTRUCT()
struct FLevelSequenceBindingReference
{
	GENERATED_BODY();

	/**
	 * Default construction only used for serialization
	 */
	FLevelSequenceBindingReference() {}

	/**
	 * Construct a new binding reference from an object, and a given context (expected to be either a UWorld, or an AActor)
	 */
	UE_DEPRECATED(5.4, "This class is now deprecated. Please convert your code to use the more generic FMovieSceneBindingReferences.")
	LEVELSEQUENCE_API FLevelSequenceBindingReference(UObject* InObject, UObject* InContext);

	/**
	 * Structure that stores additional params that are used for resolving bindings.
	 */
	struct FResolveBindingParams
	{
		UE_DEPRECATED(5.4, "This class is now deprecated. Please convert your code to use the more generic FMovieSceneBindingReferences.")
		FResolveBindingParams() : WorldPartitionResolveData(nullptr), StreamingWorld(nullptr) {}

		// The path to the streamed level asset that contains the level sequence actor playing back the sequence. 'None' for any non - instance - level setups.
		FTopLevelAssetPath StreamedLevelAssetPath;

		// World Partition Resolve Data
		const FWorldPartitionResolveData* WorldPartitionResolveData;

		// World Partition Streaming World
		UWorld* StreamingWorld;
	};

	/**
	 * Resolve this reference within the specified context
	 *
	 * @param	InContext	The context to resolve the binding within. Either a UWorld, ULevel (when playing in an instanced level) or an AActor where this binding relates to an actor component
	 * @param	InResolveBindingParams   The struct containing additional resolving params.
	 * @return	The object (usually an Actor or an ActorComponent).
	 */
	UE_DEPRECATED(5.4, "This class is now deprecated. Please convert your code to use the more generic FMovieSceneBindingReferences.")
	LEVELSEQUENCE_API UObject* Resolve(UObject* InContext, const FResolveBindingParams& InResolveBindingParams) const;
	
	/**
	 * Check whether this binding reference is equal to the specified object
	 */
	UE_DEPRECATED(5.4, "This class is now deprecated. Please convert your code to use the more generic FMovieSceneBindingReferences.")
	LEVELSEQUENCE_API bool operator==(const FLevelSequenceBindingReference& Other) const;

	/** Handles ExternalObjectPath fixup */
	void PostSerialize(const FArchive& Ar);


	/** Replaced by ExternalObjectPath */
	UPROPERTY()
	FString PackageName_DEPRECATED;

	/** Path to a specific actor/component inside an external package */
	UPROPERTY()
	FSoftObjectPath ExternalObjectPath;

	/** Object path relative to a passed in context object, this is used if ExternalObjectPath is invalid */
	UPROPERTY()
	FString ObjectPath;
};


template<>
struct TStructOpsTypeTraits<FLevelSequenceBindingReference> : public TStructOpsTypeTraitsBase2<FLevelSequenceBindingReference>
{
	enum
	{
		WithPostSerialize = true,
	};
};

/**
 * An array of binding references
 */
USTRUCT()
struct FLevelSequenceBindingReferenceArray
{
	GENERATED_BODY()

	UPROPERTY()
	TArray<FLevelSequenceBindingReference> References;
};

USTRUCT()
struct FUpgradedLevelSequenceBindingReferences : public FMovieSceneBindingReferences
{
	GENERATED_BODY()

	void AddBinding(const FGuid& ObjectId, UObject* InObject, UObject* InContext);

	bool SerializeFromMismatchedTag(const FPropertyTag& Tag, FStructuredArchive::FSlot Slot);
};

template<>
struct TStructOpsTypeTraits<FUpgradedLevelSequenceBindingReferences> : public TStructOpsTypeTraitsBase2<FUpgradedLevelSequenceBindingReferences>
{
	enum
	{
		WithStructuredSerializeFromMismatchedTag = true,
	};
};

/**
 * Structure that stores a one to many mapping from object binding ID, to object references that pertain to that ID.
 */
USTRUCT()
struct FLevelSequenceBindingReferences
{
	GENERATED_BODY()

	/** The map from object binding ID to an array of references that pertain to that ID */
	UPROPERTY()
	TMap<FGuid, FLevelSequenceBindingReferenceArray> BindingIdToReferences;

	/** A set of object binding IDs that relate to anim sequence instances (must be a child of USkeletalMeshComponent) */
	UPROPERTY()
	TSet<FGuid> AnimSequenceInstances;

	/** A set of object binding IDs that relate to post process instances (must be a child of USkeletalMeshComponent) */
	UPROPERTY()
	TSet<FGuid> PostProcessInstances;
};


