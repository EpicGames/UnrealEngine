// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Templates/SubclassOf.h"
#include "LevelSequencePlayer.h"
#include "LevelSequenceActor.h"
#include "Blueprint/UserWidget.h"
#include "LevelSequenceBurnIn.generated.h"

/**
 * Base class for level sequence burn ins
 */
UCLASS(MinimalAPI)
class ULevelSequenceBurnIn : public UUserWidget
{
public:
	GENERATED_BODY()

	LEVELSEQUENCE_API ULevelSequenceBurnIn(const FObjectInitializer& ObjectInitializer);

	/** Initialize this burn in */
	LEVELSEQUENCE_API void TakeSnapshotsFrom(ALevelSequenceActor& InActor);

	/** Called when this burn in is receiving its settings */
	UFUNCTION(BlueprintImplementableEvent, Category="Burn In")
	LEVELSEQUENCE_API void SetSettings(UObject* InSettings);

	/** Get the settings class to use for this burn in */
	UFUNCTION(BlueprintNativeEvent, Category="Burn In")
	LEVELSEQUENCE_API TSubclassOf<ULevelSequenceBurnInInitSettings> GetSettingsClass() const;
	virtual TSubclassOf<ULevelSequenceBurnInInitSettings> GetSettingsClass_Implementation() const { return nullptr; }

protected:

	/** Called as part of the game tick loop when the sequence has been updated */
	LEVELSEQUENCE_API void OnSequenceUpdated(const UMovieSceneSequencePlayer& Player, FFrameTime CurrentTime, FFrameTime PreviousTime);

protected:

	/** Snapshot of frame information. */
	UPROPERTY(BlueprintReadOnly, Category="Burn In")
	FLevelSequencePlayerSnapshot FrameInformation;

	/** The actor to get our burn in frames from */
	UPROPERTY(BlueprintReadOnly, Category="Burn In")
	TObjectPtr<ALevelSequenceActor> LevelSequenceActor;
};
