// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "UObject/ObjectMacros.h"
#include "Engine/EngineTypes.h"
#include "LevelSequenceCameraSettings.generated.h"

USTRUCT(BlueprintType)
struct FLevelSequenceCameraSettings
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Aspect Ratio")
	bool bOverrideAspectRatioAxisConstraint = true;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Aspect Ratio", meta = (EditCondition = bOverrideAspectRatioAxisConstraint))
	TEnumAsByte<EAspectRatioAxisConstraint> AspectRatioAxisConstraint = EAspectRatioAxisConstraint::AspectRatio_MaintainXFOV;
};