// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "MovieSceneClock.h"
#include "Engine/Blueprint.h"
#include "Engine/BlueprintGeneratedClass.h"
#include "EntitySystem/MovieSceneSequenceInstance.h"
#include "Evaluation/IMovieScenePlaybackCapability.h"
#include "MovieSceneSequenceID.h"
#include "MovieSceneObjectBindingID.h"
#include "Misc/QualifiedFrameTime.h"
#include "LevelSequenceDirector.generated.h"

class IMovieScenePlayer;
class ULevelSequencePlayer;
class UMovieSceneEntitySystemLinker;

UCLASS(Blueprintable, MinimalAPI)
class ULevelSequenceDirector : public UObject
{
public:
	GENERATED_BODY()

	/** Called when this director is created */
	UFUNCTION(BlueprintImplementableEvent, Category="Sequencer")
	LEVELSEQUENCE_API void OnCreated();

	/**
	 * 
	 * @return The custom clock of the outermost (root) sequence
	 */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Director")
	LEVELSEQUENCE_API UMovieSceneClock* GetRootSequenceCustomClock() const;

	/**
	 * 
	 * @return The custom clock of this director's sequence
	 */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Director")
	LEVELSEQUENCE_API UMovieSceneClock* GetSequenceCustomClock() const;
	
	/**
	 * Get the current time for the outermost (root) sequence
	 * @return The current playback position of the outermost (root) sequence
	 */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Director")
	LEVELSEQUENCE_API FQualifiedFrameTime GetRootSequenceTime() const;

	/**
	 * Get the current time for this director's sub-sequence (or the root sequence, if this is a root sequence director)
	 * @return The current playback position of this director's sequence
	 */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Director")
	LEVELSEQUENCE_API FQualifiedFrameTime GetCurrentTime() const;

	/**
	 * Resolve the bindings inside this sub-sequence that relate to the specified ID
	 * @note: ObjectBinding should be constructed from the same sequence as this Sequence Director's owning Sequence (see the GetSequenceBinding node)
	 *
	 * @param ObjectBinding The ID for the object binding inside this sub-sequence or one of its children to resolve
	 */
	UFUNCTION(BlueprintCallable, Category="Sequencer|Director")
	LEVELSEQUENCE_API TArray<UObject*> GetBoundObjects(FMovieSceneObjectBindingID ObjectBinding);


	/**
	 * Resolve the first valid binding inside this sub-sequence that relates to the specified ID
	 * @note: ObjectBinding should be constructed from the same sequence as this Sequence Director's owning Sequence (see the GetSequenceBinding node)
	 *
	 * @param ObjectBinding The ID for the object binding inside this sub-sequence or one of its children to resolve
	 */
	UFUNCTION(BlueprintCallable, Category="Sequencer|Director")
	LEVELSEQUENCE_API UObject* GetBoundObject(FMovieSceneObjectBindingID ObjectBinding);


	/**
	 * Resolve the actor bindings inside this sub-sequence that relate to the specified ID
	 * @note: ObjectBinding should be constructed from the same sequence as this Sequence Director's owning Sequence (see the GetSequenceBinding node)
	 *
	 * @param ObjectBinding The ID for the object binding inside this sub-sequence or one of its children to resolve
	 */
	UFUNCTION(BlueprintCallable, Category="Sequencer|Director")
	LEVELSEQUENCE_API TArray<AActor*> GetBoundActors(FMovieSceneObjectBindingID ObjectBinding);


	/**
	 * Resolve the first valid Actor binding inside this sub-sequence that relates to the specified ID
	 * @note: ObjectBinding should be constructed from the same sequence as this Sequence Director's owning Sequence (see the GetSequenceBinding node)
	 *
	 * @param ObjectBinding The ID for the object binding inside this sub-sequence or one of its children to resolve
	 */
	UFUNCTION(BlueprintCallable, Category="Sequencer|Director")
	LEVELSEQUENCE_API AActor* GetBoundActor(FMovieSceneObjectBindingID ObjectBinding);

	/*
	 * Get the current sequence that this director is playing back within 
	 */
	UFUNCTION(BlueprintCallable, Category="Sequencer|Director")
	LEVELSEQUENCE_API UMovieSceneSequence* GetSequence();

public:

	LEVELSEQUENCE_API virtual UWorld* GetWorld() const override;

private:

	const UE::MovieScene::FSequenceInstance* FindSequenceInstance() const;

public:

	/** The Sequence ID for the sequence this director is playing back within - has to be stored as an int32 so that it is reinstanced correctly*/
	UPROPERTY()
	int32 SubSequenceID;

	/** The linker inside which the sequence is evaluating. Only valid in game or in PIE/Simulate. */
	UPROPERTY()
	TWeakObjectPtr<UMovieSceneEntitySystemLinker> WeakLinker;

	/** Instance ID of the sequence. Only valid in game or in PIE/Simulate. */
	UPROPERTY()
	uint16 InstanceID = (uint16)-1;

	/** Instance serial of the sequence. Only valid in game or in PIE/Simulate. */
	UPROPERTY()
	uint16 InstanceSerial = 0;

	/** Pointer to the player that's playing back this director's sequence. Only valid in game or in PIE/Simulate. */
	UPROPERTY(BlueprintReadOnly, Category="Cinematics")
	TObjectPtr<ULevelSequencePlayer> Player;

	/** Native player interface index - stored by index so that it can be reinstanced correctly */
	UPROPERTY()
	int32 MovieScenePlayerIndex;
};


UCLASS()
class ULegacyLevelSequenceDirectorBlueprint : public UBlueprint
{
	GENERATED_BODY()

	ULegacyLevelSequenceDirectorBlueprint(const FObjectInitializer& ObjInit)
		: Super(ObjInit)
	{
		ParentClass = ULevelSequenceDirector::StaticClass();
	}
};
