// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Misc/Guid.h"
#include "UObject/Class.h"
#include "LevelSequenceLegacyObjectReference.generated.h"

/**
 * Legacy method by which objects were referenced within a level sequence. No longer used. See FLevelSequenceBindingReference for up-to-date implementation.
 */
USTRUCT()
struct FLevelSequenceLegacyObjectReference
{
	GENERATED_BODY();

	/**
	 * Resolve this reference within the specified context
	 *
	 * @return The object (usually an Actor or an ActorComponent).
	 */
	UObject* Resolve(UObject* InContext) const;

	/**
	 * Serialization operator
	 */
	friend FArchive& operator<<(FArchive& Ar, FLevelSequenceLegacyObjectReference& Ref)
	{
		Ar << Ref.ObjectId;
		Ar << Ref.ObjectPath;
		return Ar;
	}

	/** Primary method of resolution - object ID, stored as an annotation on the object in the world */
	FUniqueObjectGuid ObjectId;

	/** Secondary method of resolution - path to the object within the context */
	FString ObjectPath;
};

USTRUCT()
struct FLevelSequenceObjectReferenceMap
{
public:

	GENERATED_BODY();
	
	/**
	 * Serialization function
	 */
	bool Serialize(FArchive& Ar);

	TMap<FGuid, FLevelSequenceLegacyObjectReference> Map;
};

template<> struct TStructOpsTypeTraits<FLevelSequenceObjectReferenceMap> : public TStructOpsTypeTraitsBase2<FLevelSequenceObjectReferenceMap>
{
	enum { WithSerializer = true };
};