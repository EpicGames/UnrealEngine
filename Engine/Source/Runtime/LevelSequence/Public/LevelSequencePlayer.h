// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Engine/EngineTypes.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "UObject/ScriptMacros.h"
#include "Evaluation/CameraCutPlaybackCapability.h"
#include "Evaluation/MovieScenePlayback.h"
#include "Evaluation/MovieSceneEvaluationTemplateInstance.h"
#include "Evaluation/PersistentEvaluationData.h"
#include "MovieSceneSequencePlayer.h"
#include "Misc/QualifiedFrameTime.h"
#include "LevelSequence.h"
#include "LevelSequenceCameraSettings.h"
#include "WorldPartition/WorldPartitionActorContainerID.h"
#include "LevelSequencePlayer.generated.h"

class AActor;
class ALevelSequenceActor;
class FLevelSequenceSpawnRegister;
class FViewportClient;
class UCameraComponent;

DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FOnLevelSequencePlayerCameraCutEvent , UCameraComponent*, CameraComponent);

/**
 * Frame snapshot information for a level sequence
 */
USTRUCT(BlueprintType)
struct FLevelSequencePlayerSnapshot
{
	GENERATED_BODY()

	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category="General")
	FString RootName;

	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category="General")
	FQualifiedFrameTime RootTime;

	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category="General")
	FQualifiedFrameTime SourceTime;

	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category="General")
	FString CurrentShotName;

	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category="General")
	FQualifiedFrameTime CurrentShotLocalTime;

	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category="General")
	FQualifiedFrameTime CurrentShotSourceTime;

	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category="General")
	FString SourceTimecode;

	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category="General")
	TSoftObjectPtr<UCameraComponent> CameraComponent;

	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = "General")
	TObjectPtr<ULevelSequence> ActiveShot = nullptr;

	UPROPERTY()
	FMovieSceneSequenceID ShotID;
};

/**
 * ULevelSequencePlayer is used to actually "play" an level sequence asset at runtime.
 *
 * This class keeps track of playback state and provides functions for manipulating
 * an level sequence while its playing.
 */
UCLASS(BlueprintType, MinimalAPI)
class ULevelSequencePlayer
	: public UMovieSceneSequencePlayer
	, public UE::MovieScene::FCameraCutPlaybackCapability
{
public:
	LEVELSEQUENCE_API ULevelSequencePlayer(const FObjectInitializer&);

	GENERATED_BODY()

	/**
	 * Initialize the player.
	 *
	 * @param InLevelSequence The level sequence to play.
	 * @param InLevel The level that the animation is played in.
	 * @param InCameraSettings The desired camera settings
	 */
	LEVELSEQUENCE_API void Initialize(ULevelSequence* InLevelSequence, ULevel* InLevel, const FLevelSequenceCameraSettings& InCameraSettings);

	LEVELSEQUENCE_API void SetSourceActorContext(UWorld* InStreamingWorld, FActorContainerID InContainerID, FTopLevelAssetPath InSourceAssetPath);

public:

	/**
	 * Create a new level sequence player.
	 *
	 * @param WorldContextObject Context object from which to retrieve a UWorld.
	 * @param LevelSequence The level sequence to play.
	 * @param Settings The desired playback settings
	 * @param OutActor The level sequence actor created to play this sequence.
	 */
	UFUNCTION(BlueprintCallable, Category="Sequencer|Player", meta=(WorldContext="WorldContextObject", DynamicOutputParam="OutActor"))
	static LEVELSEQUENCE_API ULevelSequencePlayer* CreateLevelSequencePlayer(UObject* WorldContextObject, ULevelSequence* LevelSequence, FMovieSceneSequencePlaybackSettings Settings, ALevelSequenceActor*& OutActor);

	/** Event triggered when there is a camera cut */
	UPROPERTY(BlueprintAssignable, Category="Sequencer|Player")
	FOnLevelSequencePlayerCameraCutEvent OnCameraCut;

	/** Get the active camera cut camera */
	UFUNCTION(BlueprintCallable, Category="Sequencer|Player")
	UCameraComponent* GetActiveCameraComponent() const { return CachedCameraComponent.Get(); }

public:

	// IMovieScenePlayer interface
	LEVELSEQUENCE_API virtual UObject* GetPlaybackContext() const override;
	LEVELSEQUENCE_API virtual TArray<UObject*> GetEventContexts() const override;

	LEVELSEQUENCE_API void RewindForReplay();

protected:

	// IMovieScenePlayer interface
	
	LEVELSEQUENCE_API virtual void ResolveBoundObjects(UE::UniversalObjectLocator::FResolveParams& ResolveParams, const FGuid& InBindingId, FMovieSceneSequenceID SequenceID, UMovieSceneSequence& Sequence, TArray<UObject*, TInlineAllocator<1>>& OutObjects) const override;
	LEVELSEQUENCE_API virtual void InitializeRootInstance(TSharedRef<UE::MovieScene::FSharedPlaybackState> NewSharedPlaybackState) override;

	//~ UMovieSceneSequencePlayer interface
	LEVELSEQUENCE_API virtual bool CanPlay() const override;
	LEVELSEQUENCE_API virtual void OnStartedPlaying() override;
	LEVELSEQUENCE_API virtual void OnStopped() override;
	LEVELSEQUENCE_API virtual void UpdateMovieSceneInstance(FMovieSceneEvaluationRange InRange, EMovieScenePlayerStatus::Type PlayerStatus, const FMovieSceneUpdateArgs& Args) override;

	//~ FCameraCutPlaybackCapability interface
	LEVELSEQUENCE_API virtual bool ShouldUpdateCameraCut() override;
	LEVELSEQUENCE_API virtual float GetCameraBlendPlayRate() override;
	LEVELSEQUENCE_API virtual TOptional<EAspectRatioAxisConstraint> GetAspectRatioAxisConstraintOverride() override;
	LEVELSEQUENCE_API virtual void OnCameraCutUpdated(const UE::MovieScene::FOnCameraCutUpdatedParams& Params) override;

public:

	/** Populate the specified array with any given event contexts for the specified world */
	static LEVELSEQUENCE_API void GetEventContexts(UWorld& InWorld, TArray<UObject*>& OutContexts);

	/** Take a snapshot of the current state of this player */
	LEVELSEQUENCE_API void TakeFrameSnapshot(FLevelSequencePlayerSnapshot& OutSnapshot) const;

	/** Set the offset time for the snapshot in play rate frames. */
	void SetSnapshotOffsetFrames(int32 InFrameOffset) { SnapshotOffsetTime = TOptional<int32>(InFrameOffset); }

private:

	LEVELSEQUENCE_API void EnableCinematicMode(bool bEnable);

	void InitializeLevelSequenceRootInstance(TSharedRef<UE::MovieScene::FSharedPlaybackState> NewSharedPlaybackState);

private:

	/** The world this player will spawn actors in, if needed */
	TWeakObjectPtr<UWorld> World;

	/** The world this player will spawn actors in, if needed */
	TWeakObjectPtr<ULevel> Level;

	/** The camera settings to use when playing the sequence */
	FLevelSequenceCameraSettings CameraSettings;

protected:

	TOptional<int32> SnapshotOffsetTime;

	TWeakObjectPtr<UCameraComponent> CachedCameraComponent;

private:

	TOptional<FLevelSequencePlayerSnapshot> PreviousSnapshot;

	/** Optional streaming world that should be used primarily for resolving actor references. Used for locating actors within World Partition runtime cells. */
	TWeakObjectPtr<UWorld> WeakStreamingWorld;
	/** Source asset path denoting the level asset path that has been streamed in. */
	FTopLevelAssetPath SourceAssetPath;
	/** World Partition container ID for the world that should be added to any actor locaters when being resolved within the same world. */
	FActorContainerID ContainerID;
};
