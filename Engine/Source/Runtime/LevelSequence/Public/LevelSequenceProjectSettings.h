// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "Engine/DeveloperSettings.h"
#include "MovieSceneFwd.h"
#include "LevelSequenceProjectSettings.generated.h"

#define UE_API LEVELSEQUENCE_API

// Settings for level sequences
UCLASS(MinimalAPI, config=Engine, defaultconfig, meta=(DisplayName="Level Sequence"))
class ULevelSequenceProjectSettings : public UDeveloperSettings
{
	GENERATED_BODY()

public:
	UE_API ULevelSequenceProjectSettings();

	UPROPERTY(config, EditAnywhere, Category=Timeline, meta=(ConsoleVariable="LevelSequence.DefaultLockEngineToDisplayRate", Tooltip="0: Playback locked to playback frames\n1: Unlocked playback with sub frame interpolation"))
	bool bDefaultLockEngineToDisplayRate;

	UPROPERTY(config, EditAnywhere, Category=Timeline, meta=(ConsoleVariable="LevelSequence.DefaultDisplayRate", Tooltip="Specifies default display frame rate for newly created level sequences; also defines frame locked frame rate where sequences are set to be frame locked. Examples: 30 fps, 120/1 (120 fps), 30000/1001 (29.97), 0.01s (10ms)."))
	FString DefaultDisplayRate;

	UPROPERTY(config, EditAnywhere, Category=Timeline, meta=(ConsoleVariable="LevelSequence.DefaultTickResolution", Tooltip="Specifies default tick resolution for newly created level sequences. Examples: 30 fps, 120/1 (120 fps), 30000/1001 (29.97), 0.01s (10ms)."))
	FString DefaultTickResolution;

	UPROPERTY(config, EditAnywhere, Category=Timeline, meta=(ConsoleVariable="LevelSequence.DefaultClockSource", Tooltip="Specifies default clock source for newly created level sequences. Examples: 0: Tick, 1: Platform, 2: Audio, 3: RelativeTimecode, 4: Timecode, 5: Custom"))
	EUpdateClockSource DefaultClockSource;

	UE_API virtual void PostInitProperties() override;
#if WITH_EDITOR
	UE_API virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
#endif
};

#undef UE_API
