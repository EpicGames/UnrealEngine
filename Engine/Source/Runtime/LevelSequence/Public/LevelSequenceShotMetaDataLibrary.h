// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AssetRegistry/AssetData.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "LevelSequenceShotMetaDataLibrary.generated.h"

#define UE_API LEVELSEQUENCE_API

class ULevelSequence;
struct FAssetData;

/**
 * Manages ULevelSequence meta data that is common for rating shots in cinematic workflows.
 * Manages access to UMovieSceneShotsMetaData for ULevelSequences.
 */
UCLASS(MinimalAPI)
class ULevelSequenceShotMetaDataLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()
public:
	
	/**
	 * Gets whether the user has marked this shot as no good.
	 * @return Whether the value was set by the user.
	 */
	UFUNCTION(BlueprintPure, Category = "Level Sequence")
	static UE_API bool GetIsNoGood(const ULevelSequence* InLevelSequence, bool& bOutNoGood);
	/**
	 * Gets whether this was flagged by a user.
	 * @return Whether the value was set by the user.
	 */
	UFUNCTION(BlueprintPure, Category = "Level Sequence")
	static UE_API bool GetIsFlagged(const ULevelSequence* InLevelSequence, bool& bOutIsFlagged);
	/**
	 * Gets whether this was recorded.
	 * @return Whether the value was set.
	 */
	UFUNCTION(BlueprintPure, Category = "Level Sequence")
	static UE_API bool GetIsRecorded(const ULevelSequence* InLevelSequence, bool& bOutIsRecorded);
	/**
	 * Gets whether this was recorded as a subsequence.
	 * @return Whether the value was set.
	 */
	UFUNCTION(BlueprintPure, Category = "Level Sequence")
	static UE_API bool GetIsSubSequence(const ULevelSequence* InLevelSequence, bool& bOutIsSubSequence);
	/**
	 * Gets the favorite rating. The favorite rating is like a star rating, usually 1-3 if it was rated.
	 * @return Whether the value was set by the user.
	 */
	UFUNCTION(BlueprintPure, Category = "Level Sequence")
	static UE_API bool GetFavoriteRating(const ULevelSequence* InLevelSequence, int32& OutFavoriteRating);

	/** @return Whether the user has ever set a value for this property. */
	UFUNCTION(BlueprintPure, Category = "Level Sequence")
	static UE_API bool HasIsNoGood(const ULevelSequence* InLevelSequence);
	/** @return Whether the user has ever set a value for this property. */
	UFUNCTION(BlueprintPure, Category = "Level Sequence")
	static UE_API bool HasIsFlagged(const ULevelSequence* InLevelSequence);
	/** @return Whether IsRecorded has been set. */
	UFUNCTION(BlueprintPure, Category = "Level Sequence")
	static UE_API bool HasIsRecorded(const ULevelSequence* InLevelSequence);
	/** @return Whether IsSubSequence has been set. */
	UFUNCTION(BlueprintPure, Category = "Level Sequence")
	static UE_API bool HasIsSubSequence(const ULevelSequence* InLevelSequence);
	/** @return Whether the user has ever set a value for this property. */
	UFUNCTION(BlueprintPure, Category = "Level Sequence")
	static UE_API bool HasFavoriteRating(const ULevelSequence* InLevelSequence);

	/** Sets whether the user has marked this shot as no good. */
	UFUNCTION(BlueprintCallable, Category = "Level Sequence")
	static UE_API void SetIsNoGood(ULevelSequence* InLevelSequence, bool bInIsNoGood);
	/** Sets whether this was flagged by a user. */
	UFUNCTION(BlueprintCallable, Category = "Level Sequence")
	static UE_API void SetIsFlagged(ULevelSequence* InLevelSequence, bool bInIsFlagged);
	/** Sets whether this sequence was recorded. */
	UFUNCTION(BlueprintCallable, Category = "Level Sequence")
	static UE_API void SetIsRecorded(ULevelSequence* InLevelSequence, bool bInIsRecorded);
	/** Sets whether this is a subsequence. */
	UFUNCTION(BlueprintCallable, Category = "Level Sequence")
	static UE_API void SetIsSubSequence(ULevelSequence* InLevelSequence, bool bInIsSubSequence);
	/** Sets the favorite rating. The favorite rating is like a star rating, usually 1-3 if it was rated. */
	UFUNCTION(BlueprintCallable, Category = "Level Sequence")
	static UE_API void SetFavoriteRating(ULevelSequence* InLevelSequence, int32 InFavoriteRating);

	/** Clear the IsNoGood flag. */
	UFUNCTION(BlueprintCallable, Category = "Level Sequence")
	static UE_API void ClearIsNoGood(ULevelSequence* InLevelSequence);
	/** Clear IsFlagged flag. */
	UFUNCTION(BlueprintCallable, Category = "Level Sequence")
	static UE_API void ClearIsFlagged(ULevelSequence* InLevelSequence);
	/** Clear IsRecorded flag. */
	UFUNCTION(BlueprintCallable, Category = "Level Sequence")
	static UE_API void ClearIsRecorded(ULevelSequence* InLevelSequence);
	/** Clear IsSubSequence flag. */
	UFUNCTION(BlueprintCallable, Category = "Level Sequence")
	static UE_API void ClearIsSubSequence(ULevelSequence* InLevelSequence);
	/** Clears the favorite rating. */
	UFUNCTION(BlueprintCallable, Category = "Level Sequence")
	static UE_API void ClearFavoriteRating(ULevelSequence* InLevelSequence);

	/** @return The asset tag for the IsNoGood value. */
	UFUNCTION(BlueprintPure, Category = "Level Sequence")
	static UE_API FName GetIsNoGoodAssetTag();
	/** @return The asset tag for the IsFlagged value. */
	UFUNCTION(BlueprintPure, Category = "Level Sequence")
	static UE_API FName GetIsFlaggedAssetTag();
	/** @return The asset tag for the IsRecorded value. */
	UFUNCTION(BlueprintPure, Category = "Level Sequence")
	static UE_API FName GetIsRecordedAssetTag();
	/** @return The asset tag for the IsSubSequence value. */
	UFUNCTION(BlueprintPure, Category = "Level Sequence")
	static UE_API FName GetIsSubSequenceAssetTag();
	/** @return The asset tag for the FavoriteRating value. */
	UFUNCTION(BlueprintPure, Category = "Level Sequence")
	static UE_API FName GetFavoriteRatingAssetTag();

	/**
	 * Gets whether the user has marked this shot as no good.
	 * @return Whether the value was set by the user.
	 */
	UFUNCTION(BlueprintPure, Category = "Level Sequence")
	static UE_API bool GetIsNoGoodByAssetData(const FAssetData& InAssetData, bool& bOutNoGood);
	/**
	 * Gets whether this was flagged by a user.
	 * @return Whether the value was set by the user.
	 */
	UFUNCTION(BlueprintPure, Category = "Level Sequence")
	static UE_API bool GetIsFlaggedByAssetData(const FAssetData& InAssetData, bool& bOutIsFlagged);
	/**
	 * Gets whether this was recorded.
	 * @return Whether the value was set by the user.
	 */
	UFUNCTION(BlueprintPure, Category = "Level Sequence")
	static UE_API bool GetIsRecordedByAssetData(const FAssetData& InAssetData, bool& bOutIsRecorded);
	/**
	 * Gets whether this was recorded as a sub sequence.
	 * @return Whether the value was set by the user.
	 */
	UFUNCTION(BlueprintPure, Category = "Level Sequence")
	static UE_API bool GetIsSubSequenceByAssetData(const FAssetData& InAssetData, bool& bOutIsSubSequence);
	/**
	 * Gets the favorite rating. The favorite rating is like a star rating, usually 1-3 if it was rated.
	 * @return Whether the value was set by the user.
	 */
	UFUNCTION(BlueprintPure, Category = "Level Sequence")
	static UE_API bool GetFavoriteRatingByAssetData(const FAssetData& InAssetData, int32& OutFavoriteRating);
	
	/** @return Whether the user has ever set a value for this property. */
	UFUNCTION(BlueprintPure, Category = "Level Sequence")
	static UE_API bool HasIsNoGoodByAssetData(const FAssetData& InAssetData);
	/** @return Whether the user has ever set a value for this property. */
	UFUNCTION(BlueprintPure, Category = "Level Sequence")
	static UE_API bool HasIsFlaggedByAssetData(const FAssetData& InAssetData);
	/** @return Whether the user has ever set a value for this property. */
	UFUNCTION(BlueprintPure, Category = "Level Sequence")
	static UE_API bool HasIsRecordedByAssetData(const FAssetData& InAssetData);
	/** @return Whether the user has ever set a value for this property. */
	UFUNCTION(BlueprintPure, Category = "Level Sequence")
	static UE_API bool HasIsSubSequenceByAssetData(const FAssetData& InAssetData);
	/** @return Whether the user has ever set a value for this property. */
	UFUNCTION(BlueprintPure, Category = "Level Sequence")
	static UE_API bool HasFavoriteRatingByAssetData(const FAssetData& InAssetData);
};

#undef UE_API
