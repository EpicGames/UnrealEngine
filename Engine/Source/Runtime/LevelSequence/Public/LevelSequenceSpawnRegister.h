// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "CoreMinimal.h"
#include "MovieSceneSequenceID.h"
#include "MovieSceneSpawnRegister.h"
#include "Templates/SharedPointer.h"
#include "UObject/Class.h"

class IMovieSceneObjectSpawner;
class IMovieScenePlayer;
class UClass;
class UObject;
struct FMovieSceneSpawnable;
class UMovieSceneSpawnableBindingBase;

/** Movie scene spawn register that knows how to handle spawning objects (actors) for a level sequence  */
class FLevelSequenceSpawnRegister : public FMovieSceneSpawnRegister
{
public:
	LEVELSEQUENCE_API FLevelSequenceSpawnRegister();
	LEVELSEQUENCE_API FLevelSequenceSpawnRegister(const FLevelSequenceSpawnRegister&);
	LEVELSEQUENCE_API ~FLevelSequenceSpawnRegister();

protected:
	/** ~ FMovieSceneSpawnRegister interface */
	LEVELSEQUENCE_API virtual UObject* SpawnObject(FMovieSceneSpawnable& Spawnable, FMovieSceneSequenceIDRef TemplateID, TSharedRef<const FSharedPlaybackState> SharedPlaybackState) override;
	LEVELSEQUENCE_API virtual void DestroySpawnedObject(UObject& Object, UMovieSceneSpawnableBindingBase* CustomSpawnableBinding) override;

#if WITH_EDITOR
	LEVELSEQUENCE_API virtual bool CanSpawnObject(UClass* InClass) const override;
#endif

protected:
	/** Extension object spawners */
	TArray<TSharedRef<IMovieSceneObjectSpawner>> MovieSceneObjectSpawners;
};
