// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "UObject/ObjectMacros.h"
#include "GameFramework/Actor.h"
#include "Evaluation/IMovieSceneCustomClockSource.h"
#include "LevelSequenceActor.h"
#include "SequenceMediaController.generated.h"

enum class EMediaEvent;

class FDelegateHandle;
class UMediaPlayer;

/**
 * Replicated actor class that is responsible for instigating various cinematic assets (Media, Audio, Level Sequences) in a synchronized fasion
 */
UCLASS(hideCategories=(Rendering, Physics, HLOD, Activation, Input, Collision), MinimalAPI)
class ALevelSequenceMediaController
	: public AActor
	, public IMovieSceneCustomClockSource
{
public:

	GENERATED_BODY()

	LEVELSEQUENCE_API ALevelSequenceMediaController(const FObjectInitializer& Init);

	UFUNCTION(BlueprintCallable, Category="Synchronization")
	LEVELSEQUENCE_API void Play();

	/**
	 * Access this actor's media component
	 */
	UFUNCTION(BlueprintPure, Category="Synchronization")
	UMediaComponent* GetMediaComponent() const
	{
		return MediaComponent;
	}

	/**
	 * Access this actor's Level Sequence Actor
	 */
	UFUNCTION(BlueprintPure, Category="Synchronization")
	ALevelSequenceActor* GetSequence() const
	{
		return Sequence;
	}

	/**
	 * Forcibly synchronize the sequence to the server's position if it has diverged by more than the specified threshold
	 */
	UFUNCTION(BlueprintCallable, Category="Synchronization")
	LEVELSEQUENCE_API void SynchronizeToServer(float DesyncThresholdSeconds = 2.f);

private:

	LEVELSEQUENCE_API void Client_Play();

	LEVELSEQUENCE_API void Client_ConditionallyForceTime(float DesyncThresholdSeconds);

private:

	LEVELSEQUENCE_API virtual void GetLifetimeReplicatedProps(TArray<FLifetimeProperty>& OutLifetimeProps) const;

private:

	LEVELSEQUENCE_API virtual void OnTick(float DeltaSeconds, float InPlayRate) override final;
	LEVELSEQUENCE_API virtual void OnStartPlaying(const FQualifiedFrameTime& InStartTime) override final;
	LEVELSEQUENCE_API virtual void OnStopPlaying(const FQualifiedFrameTime& InStopTime) override final;
	LEVELSEQUENCE_API virtual FFrameTime OnRequestCurrentTime(const FQualifiedFrameTime& InCurrentTime, float InPlayRate) override final;

private:

	UFUNCTION()
	LEVELSEQUENCE_API void OnRep_ServerStartTimeSeconds();

	/** Pointer to the sequence actor to use for playback */
	UPROPERTY(EditAnywhere, BlueprintGetter=GetSequence, Category="Synchronization")
	TObjectPtr<ALevelSequenceActor> Sequence;

	/** Media component that contains the media player to synchronize with */
	UPROPERTY(VisibleAnywhere, BlueprintGetter=GetMediaComponent, Category="Synchronization")
	TObjectPtr<UMediaComponent> MediaComponent;

	/** Replicated time at which the server started the sequence (taken from AGameStateBase::GetServerWorldTimeSeconds) */
	UPROPERTY(BlueprintReadOnly, Category="Synchronization", replicated, ReplicatedUsing=OnRep_ServerStartTimeSeconds, meta=(AllowPrivateAccess="true"))
	float ServerStartTimeSeconds;

	/** Time to use for the sequence playback position */
	double SequencePositionSeconds;
};
